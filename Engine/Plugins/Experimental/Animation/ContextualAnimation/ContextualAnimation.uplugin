{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "ContextualAnimation",
	"Description": "",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ContextualAnimation",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ContextualAnimationEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "MotionWarping",
			"Enabled": true
		},
		{
			"Name": "IKRig",
			"Enabled": true
		}
	]
}