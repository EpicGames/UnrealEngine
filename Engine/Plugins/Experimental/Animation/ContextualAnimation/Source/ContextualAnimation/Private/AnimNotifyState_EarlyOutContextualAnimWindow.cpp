// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyState_EarlyOutContextualAnimWindow.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "ContextualAnimSceneActorComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_EarlyOutContextualAnimWindow)

UAnimNotifyState_EarlyOutContextualAnimWindow::UAnimNotifyState_EarlyOutContextualAnimWindow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

void UAnimNotifyState_EarlyOutContextualAnimWindow::NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference)
{
	if(MeshComp)
	{
		ACharacter* CharacterOwner = Cast<ACharacter>(MeshComp->GetOwner());
		if(CharacterOwner && CharacterOwner->IsLocallyControlled())
		{
			if(UCharacterMovementComponent* MoveComp = CharacterOwner->GetCharacterMovement())
			{
				if(!MoveComp->GetLastInputVector().IsNearlyZero())
				{
					UContextualAnimSceneActorComponent* Comp = CharacterOwner->FindComponentByClass<UContextualAnimSceneActorComponent>();
					if (Comp)
					{
						Comp->EarlyOutContextualAnimScene(bStopEveryone);
					}
				}
			}
		}
		
	}
}

FString UAnimNotifyState_EarlyOutContextualAnimWindow::GetNotifyName_Implementation() const
{
	return FString(TEXT("Early Out Contextual Anim"));
}
