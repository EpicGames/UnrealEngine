// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNotifyState_IKWindow.h"
#include "Animation/AnimMontage.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_IKWindow)

void UAnimNotifyState_IKWindow::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Note this happens before serialization
			BlendIn.SetBlendOption(EAlphaBlendOption::Linear);
			BlendOut.SetBlendOption(EAlphaBlendOption::Linear);
		}
	}
	
	Super::Serialize(Ar);
}

UAnimNotifyState_IKWindow::UAnimNotifyState_IKWindow(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendIn.SetBlendTime(0.25f);
	BlendOut.SetBlendTime(0.25f);
}

FString UAnimNotifyState_IKWindow::GetNotifyName_Implementation() const
{
	return FString::Printf(TEXT("IK (%s)"), *GoalName.ToString());
}

float UAnimNotifyState_IKWindow::GetIKAlphaValue(const FName& GoalName, const FAnimMontageInstance* MontageInstance)
{
	if (MontageInstance && MontageInstance->Montage)
	{
		for (const FAnimNotifyEvent& NotifyEvent : MontageInstance->Montage->Notifies)
		{
			const UAnimNotifyState_IKWindow* Notify = NotifyEvent.NotifyStateClass ? Cast<const UAnimNotifyState_IKWindow>(NotifyEvent.NotifyStateClass) : nullptr;
			if (Notify && Notify->bEnable && Notify->GoalName == GoalName)
			{
				const float PlayLength = MontageInstance->Montage->GetPlayLength();
				const float StartTime = FMath::Clamp(NotifyEvent.GetTriggerTime(), 0.f, PlayLength);
				const float EndTime = FMath::Clamp(NotifyEvent.GetEndTriggerTime(), 0.f, PlayLength);

				if (FMath::IsWithinInclusive(MontageInstance->GetPosition(), StartTime, EndTime))
				{
					const float WindowLength = NotifyEvent.GetDuration();
					const float LocalPos = FMath::GetMappedRangeValueClamped(FVector2D(StartTime, EndTime), FVector2D(0.f, WindowLength), MontageInstance->GetPosition());
					const float BlendInTime = Notify->BlendIn.GetBlendTime();
					const float BlendOutTime = Notify->BlendOut.GetBlendTime();
					const float BlendOutStartTimeLocal = WindowLength - BlendOutTime;
					if (LocalPos >= BlendOutStartTimeLocal)
					{
						const float Alpha = FMath::GetMappedRangeValueClamped(FVector2D(BlendOutStartTimeLocal, WindowLength), FVector2D(0.f, 1.f), LocalPos);
						return 1.f - FAlphaBlend::AlphaToBlendOption(Alpha, Notify->BlendOut.GetBlendOption(), Notify->BlendOut.GetCustomCurve());
					}
					else if (LocalPos <= BlendInTime)
					{
						const float Alpha = FMath::GetMappedRangeValueClamped(FVector2D(0.f, BlendInTime), FVector2D(0.f, 1.f), LocalPos);
						return FAlphaBlend::AlphaToBlendOption(Alpha, Notify->BlendIn.GetBlendOption(), Notify->BlendIn.GetCustomCurve());
					}
					else
					{
						return 1.f;
					}
				}
			}
		}
	}

	return 0.f;
}
