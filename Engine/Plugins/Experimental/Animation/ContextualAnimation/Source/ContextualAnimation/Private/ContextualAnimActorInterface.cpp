// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimActorInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimActorInterface)

UContextualAnimActorInterface::UContextualAnimActorInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) 
{}
