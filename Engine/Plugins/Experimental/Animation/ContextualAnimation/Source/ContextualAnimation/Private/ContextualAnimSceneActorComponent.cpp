// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimSceneActorComponent.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "ContextualAnimSelectionCriterion.h"
#include "Components/SkeletalMeshComponent.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimUtilities.h"
#include "AnimNotifyState_IKWindow.h"
#include "GameFramework/Pawn.h"
#include "MeshElementCollector.h"
#include "Net/UnrealNetwork.h"
#include "Rig/IKRigDataTypes.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Physics/PhysicsInterfaceTypes.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveDrawingUtils.h"
#include "MotionWarpingComponent.h"
#include "Engine/World.h"
#include "SceneView.h"
#include "TimerManager.h"
#include "Engine/CollisionProfile.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimSceneActorComponent)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
TAutoConsoleVariable<int32> CVarContextualAnimIKDebug(TEXT("a.ContextualAnim.IK.Debug"), 0, TEXT("Draw Debug IK Targets"));
TAutoConsoleVariable<float> CVarContextualAnimIKDrawDebugLifetime(TEXT("a.ContextualAnim.IK.DrawDebugLifetime"), 0, TEXT("Draw Debug Duration"));
#endif

namespace ContextualAnimSceneActorComponentCVars
{
	static bool bLeaveSceneOnEndPlay = true;
	static FAutoConsoleVariableRef CVarLeaveSceneOnEndPlay(
		TEXT("a.ContextualAnim.LeaveSceneOnEndPlay"), bLeaveSceneOnEndPlay,
		TEXT("If true, will call LeaveScene on EndPlay of each actor if they're in an active scene."),
		ECVF_Default);
}

void FContextualAnimRepData::IncrementRepCounter()
{
	static uint8 Counter = 0;
	if (Counter >= UINT8_MAX)
	{
		Counter = 0;
	}
	++Counter;
	RepCounter = Counter;
}

static int32 CalculateWarpPointsForBindings(const FContextualAnimSceneBindings& Bindings, int32 SectionIdx, int32 AnimSetIdx, TArray<FContextualAnimWarpPoint>& OutWarpPoints)
{
	const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset();
	if (Asset == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "CalculateWarpPointsForBindings Invalid Scene Asset. Bindings Id: %d Bindings Num: %d SectionIdx: %d AnimSetIdx: %d", Bindings.GetID(), Bindings.Num(), SectionIdx, AnimSetIdx);
		return 0;
	}

	const FContextualAnimSceneSection* Section = Asset->GetSection(SectionIdx);
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "CalculateWarpPointsForBindings Invalid Section. Bindings Id: %d Bindings Num: %d SectionIdx: %d AnimSetIdx: %d", Bindings.GetID(), Bindings.Num(), SectionIdx, AnimSetIdx);
		return 0;
	}

	OutWarpPoints.Reset(Section->GetWarpPointDefinitions().Num());
	for (const FContextualAnimWarpPointDefinition& WarpPointDef : Section->GetWarpPointDefinitions())
	{
		FContextualAnimWarpPoint WarpPoint;
		if (Bindings.CalculateWarpPoint(WarpPointDef, WarpPoint))
		{
			OutWarpPoints.Add(WarpPoint);
		}
	}

	return OutWarpPoints.Num();
}

UContextualAnimSceneActorComponent::UContextualAnimSceneActorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	SetIsReplicatedByDefault(true);
}

void UContextualAnimSceneActorComponent::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UContextualAnimSceneActorComponent* This = CastChecked<UContextualAnimSceneActorComponent>(InThis);
	This->Bindings.AddReferencedObjects(Collector);

	Super::AddReferencedObjects(This, Collector);
}

void UContextualAnimSceneActorComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	DOREPLIFETIME_WITH_PARAMS_FAST(UContextualAnimSceneActorComponent, RepBindings, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UContextualAnimSceneActorComponent, RepLateJoinData, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UContextualAnimSceneActorComponent, RepTransitionSingleActorData, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UContextualAnimSceneActorComponent, RepTransitionData, Params);
}

void UContextualAnimSceneActorComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	if (ContextualAnimSceneActorComponentCVars::bLeaveSceneOnEndPlay && IsInActiveScene())
	{
		LeaveScene();
	}

	if (OwnerAnimInstance.IsValid())
	{
		OwnerAnimInstance->OnMontageBlendingOut.RemoveDynamic(this, &UContextualAnimSceneActorComponent::OnMontageBlendingOut);
		OwnerAnimInstance->OnPlayMontageNotifyBegin.RemoveDynamic(this, &UContextualAnimSceneActorComponent::OnPlayMontageNotifyBegin);
	}

	Super::EndPlay(EndPlayReason);
}

bool UContextualAnimSceneActorComponent::IsOwnerLocallyControlled() const
{
	if (const APawn* OwnerPawn = Cast<APawn>(GetOwner()))
	{
		return OwnerPawn->IsLocallyControlled();
	}

	return false;
}

bool UContextualAnimSceneActorComponent::IsInActiveScene() const
{
	return (Bindings.IsValid() && Bindings.FindBindingByActor(GetOwner()) != nullptr);
}

void UContextualAnimSceneActorComponent::PlayAnimation_Internal(UAnimSequenceBase* Animation, float StartTime, bool bSyncPlaybackTime)
{
	if (UAnimInstance* AnimInstance = UContextualAnimUtilities::TryGetAnimInstance(GetOwner()))
	{
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls \t\tUContextualAnimSceneActorComponent::PlayAnimation_Internal Playing Animation. Actor: %ls Anim: %ls StartTime: %f bSyncPlaybackTime: %d",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *GetNameSafe(Animation), StartTime, bSyncPlaybackTime);

		// Cache AnimInstance so we don't have to look for it in the bindings in subsequent uses.
		OwnerAnimInstance = AnimInstance;

		if (Animation)
		{
			//@TODO: Add support for dynamic montage
			UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation);

			// Keep track of this animation. Used as guarding mechanism in OnMonageBlendingOut to decide if is safe to leave the scene
			AnimsPlayed.Add(AnimMontage);

			AnimInstance->Montage_Play(AnimMontage, 1.f, EMontagePlayReturnType::MontageLength, StartTime);

			AnimInstance->OnMontageBlendingOut.AddUniqueDynamic(this, &UContextualAnimSceneActorComponent::OnMontageBlendingOut);
			AnimInstance->OnPlayMontageNotifyBegin.AddUniqueDynamic(this, &UContextualAnimSceneActorComponent::OnPlayMontageNotifyBegin);

			LoopCounter = 0;

			//@TODO: Move this out of this function
			if (bSyncPlaybackTime)
			{
				if (FAnimMontageInstance* MontageInstance = AnimInstance->GetActiveMontageInstance())
				{
					if (const FContextualAnimSceneBinding* SyncLeader = Bindings.GetSyncLeader())
					{
						if (SyncLeader->GetActor() != GetOwner())
						{
							FAnimMontageInstance* LeaderMontageInstance = SyncLeader->GetAnimMontageInstance();
							const UAnimSequenceBase* AnimSequence = SyncLeader->GetAnimation();
							if (LeaderMontageInstance && LeaderMontageInstance->Montage == AnimSequence && MontageInstance->GetMontageSyncLeader() == nullptr)
							{
								UE_LOGF(LogContextualAnim, VeryVerbose, "%-21ls \t\tUContextualAnimSceneActorComponent::PlayAnimation_Internal Syncing Animation. Actor: %ls Anim: %ls StartTime: %f bSyncPlaybackTime: %d",
									*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *GetNameSafe(Animation), StartTime, bSyncPlaybackTime);

								MontageInstance->MontageSync_Follow(LeaderMontageInstance);
							}
						}
					}
				}
			}
		}
		// if there is no animation for this actor we have to make sure we stop any currently playing animation.
		else
		{
			AnimInstance->StopAllMontages(0.f);
		}
	}

	USkeletalMeshComponent* SkelMeshComp = UContextualAnimUtilities::TryGetSkeletalMeshComponent(GetOwner());
	if (SkelMeshComp && !SkelMeshComp->OnTickPose.IsBoundToObject(this))
	{
		SkelMeshComp->OnTickPose.AddUObject(this, &UContextualAnimSceneActorComponent::OnTickPose);
	}
}

void UContextualAnimSceneActorComponent::AddOrUpdateWarpTargets(int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	// This is relevant only for character with motion warping comp
	ACharacter* CharacterOwner = Cast<ACharacter>(GetOwner());
	UMotionWarpingComponent* MotionWarpComp = CharacterOwner ? CharacterOwner->GetComponentByClass<UMotionWarpingComponent>() : nullptr;
	if (MotionWarpComp == nullptr)
	{
		return;
	}

	// Remove old warp targets to prevent actor from warping to a wrong location if calculating warp target for this interaction fails
	MotionWarpComp->RemoveAllWarpTargets();

	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(GetOwner()))
	{
		if (WarpPoints.Num() > 0)
		{
			const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset();
			if (Asset == nullptr)
			{
				UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::AddOrUpdateWarpTargets Invalid Scene Asset. Actor: %ls Bindings Id: %d Bindings Num: %d SectionIdx: %d AnimSetIdx: %d",
					*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), Bindings.GetID(), Bindings.Num(), SectionIdx, AnimSetIdx);
				return;
			}

			UAnimSequenceBase* AniamtionSequence = Binding->GetAnimation();
			if (AniamtionSequence == nullptr)
			{
				return;
			}

			for (const FContextualAnimWarpPoint& WarpPoint : WarpPoints)
			{
				if (WarpPoint.Name != NAME_None)
				{
					float StartTime, EndTime;
					UContextualAnimUtilities::GetStartAndEndTimeForWarpSection(AniamtionSequence, WarpPoint.Name, StartTime, EndTime);
					const FTransform TransformRelativeToWarpPoint = Asset->GetAlignmentTransform(*Binding, Bindings, WarpPoint.Name, EndTime);
					const FTransform WarpTargetTransform = TransformRelativeToWarpPoint * WarpPoint.Transform;
					MotionWarpComp->AddOrUpdateWarpTargetFromTransform(WarpPoint.Name, WarpTargetTransform);
					WarpTargetNamesCache.AddUnique(WarpPoint.Name);
				}
			}
		}

		const FName Role = Bindings.GetRoleFromBinding(*Binding);
		for (const FContextualAnimWarpTarget& WarpTarget : ExternalWarpTargets)
		{
			if (WarpTarget.Role == Role)
			{
				MotionWarpComp->AddOrUpdateWarpTargetFromTransform(WarpTarget.TargetName, FTransform(WarpTarget.TargetRotation, WarpTarget.TargetLocation));
				WarpTargetNamesCache.AddUnique(WarpTarget.TargetName);
			}
		}
	}
}

bool UContextualAnimSceneActorComponent::LateJoinContextualAnimScene(AActor* Actor, FName Role, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!GetOwner()->HasAuthority())
	{
		return false;
	}

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::LateJoinContextualAnimScene Invalid Bindings", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
		return false;
	}

	// Redirect the request to the leader if needed. Technically this is not necessary but the idea here is that the leader of the interaction handles all the events for that interaction
	// E.g the leader tells other actors to play the animation.
	if (const FContextualAnimSceneBinding* Leader = Bindings.GetSyncLeader())
	{
		if (Leader->GetActor() != GetOwner())
		{
			if (UContextualAnimSceneActorComponent* Comp = Leader->GetSceneActorComponent())
			{
				return Comp->LateJoinContextualAnimScene(Actor, Role, ExternalWarpTargets);
			}
		}
	}

	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::LateJoinContextualAnimScene Owner: %ls Bindings Id: %d Asset: %ls. Requester: %ls Role: %ls",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), Bindings.GetID(), *GetNameSafe(Bindings.GetSceneAsset()), *GetNameSafe(Actor), *Role.ToString());

	// For now when late joining an scene always play animation from first section
	const int32 SectionIdx = 0;
	const int32 AnimSetIdx = 0;

	// Add actor to the bindings
	if (!IsValid(Actor) || !Bindings.BindActorToRole(*Actor, Role, SectionIdx, AnimSetIdx))
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::LateJoinContextualAnimScene Failed. Reason: Adding %ls to the bindings for role: %ls failed!",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(Actor), *Role.ToString());

		return false;
	}

	// Update the bindings on all the other actors too
	for (const FContextualAnimSceneBinding& OtherBinding : Bindings)
	{
		if (OtherBinding.GetActor() != GetOwner() && OtherBinding.GetActor() != Actor)
		{
			if (UContextualAnimSceneActorComponent* Comp = OtherBinding.GetSceneActorComponent())
			{
				Comp->Bindings.BindActorToRole(*Actor, Role, SectionIdx, AnimSetIdx);
			}
		}
	}

	TArray<FContextualAnimWarpPoint> WarpPoints;
	CalculateWarpPointsForBindings(Bindings, SectionIdx, AnimSetIdx, WarpPoints);

	// Play animation and set state on this new actor that is joining us
	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(Actor))
	{
		if (UContextualAnimSceneActorComponent* Comp = Binding->GetSceneActorComponent())
		{
			Comp->LateJoinScene(Bindings, SectionIdx, AnimSetIdx, WarpPoints, ExternalWarpTargets);
		}
	}

	// Replicate late join event. See OnRep_LateJoinData
	RepLateJoinData.Actor = Actor;
	RepLateJoinData.Role = Role;
	RepLateJoinData.WarpPoints = MoveTemp(WarpPoints);
	RepLateJoinData.ExternalWarpTargets = ExternalWarpTargets;
	RepLateJoinData.IncrementRepCounter();
	MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepLateJoinData, this);
	GetOwner()->ForceNetUpdate();

	return true;
}

void UContextualAnimSceneActorComponent::LateJoinScene(const FContextualAnimSceneBindings& InBindings, int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::LateJoinScene Actor: %ls Bindings Id: %d Asset: %ls. Leaving current scene",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), Bindings.GetID(), *GetNameSafe(Bindings.GetSceneAsset()));

		LeaveScene();
	}

	if (const FContextualAnimSceneBinding* Binding = InBindings.FindBindingByActor(GetOwner()))
	{
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::LateJoinScene Actor: %ls Role: %ls Bindings Id: %d Asset: %ls",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *InBindings.GetRoleFromBinding(*Binding).ToString(), InBindings.GetID(), *GetNameSafe(InBindings.GetSceneAsset()));

		Bindings = InBindings;

		const FContextualAnimTrack* AnimTrack = Bindings.GetSceneAsset()->GetAnimTrack(SectionIdx, AnimSetIdx, Bindings.GetRoleFromBinding(*Binding));
		check(AnimTrack);


		UAnimSequenceBase* AnimSequence = Binding->GetAnimation();

		PlayAnimation_Internal(AnimSequence, 0.f, false);

		AddOrUpdateWarpTargets(SectionIdx, AnimSetIdx, WarpPoints, ExternalWarpTargets);

		SetCollisionState(*Binding);

		SetMovementState(*Binding, *AnimTrack);

		OnLateJoinScene(*Binding, SectionIdx, AnimSetIdx);

		OnJoinedSceneDelegate.Broadcast(this);
	}
}

void UContextualAnimSceneActorComponent::OnLateJoinScene(const FContextualAnimSceneBinding& Binding, int32 SectionIdx, int32 AnimSetIdx)
{
	// For derived classes to override.
}

void UContextualAnimSceneActorComponent::OnRep_LateJoinData()
{
	// This is received by the leader of the interaction on every remote client

	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnRep_LateJoinData Owner: %ls Bindings Id: %d Asset: %ls. Requester: %ls Role: %ls RepCounter: %d",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()),
		*GetNameSafe(GetOwner()), Bindings.GetID(), *GetNameSafe(Bindings.GetSceneAsset()), *GetNameSafe(RepLateJoinData.Actor.Get()), *RepLateJoinData.Role.ToString(), RepLateJoinData.RepCounter);

	if (!RepLateJoinData.IsValid())
	{
		return;
	}

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::OnRep_LateJoinData Invalid Bindings", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
		return;
	}

	AActor* Actor = RepLateJoinData.Actor.Get();
	if (Actor == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::OnRep_LateJoinData Invalid Actor in RepLateJoinData", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
		return;
	}

	const FName Role = RepLateJoinData.Role;
	const int32 SectionIdx = 0;
	const int32 AnimSetIdx = 0;

	// Add actor to the bindings
	if (!IsValid(Actor) || !Bindings.BindActorToRole(*Actor, Role, SectionIdx, AnimSetIdx))
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::OnRep_LateJoinData Failed. Reason: Adding %ls to the bindings for role: %ls failed!",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(Actor), *Role.ToString());

		return;
	}

	// Update the bindings on all the other actors too
	for (const FContextualAnimSceneBinding& OtherBinding : Bindings)
	{
		if (OtherBinding.GetActor() != GetOwner() && OtherBinding.GetActor() != Actor)
		{
			if (UContextualAnimSceneActorComponent* Comp = OtherBinding.GetSceneActorComponent())
			{
				Comp->Bindings.BindActorToRole(*Actor, Role, SectionIdx, AnimSetIdx);
			}
		}
	}

	// Play animation and set state on this new actor that is joining us
	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(Actor))
	{
		if (UContextualAnimSceneActorComponent* Comp = Binding->GetSceneActorComponent())
		{
			Comp->LateJoinScene(Bindings, SectionIdx, AnimSetIdx, RepLateJoinData.WarpPoints, RepLateJoinData.ExternalWarpTargets);
		}
	}
}

bool UContextualAnimSceneActorComponent::TransitionContextualAnimScene(FName SectionName, int32 AnimSetIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!GetOwner()->HasAuthority())
	{
		return false;
	}

	// Redirect the request to the leader if needed. Technically this is not necessary but the idea here is that the leader of the interaction handles all the events for that interaction
	// E.g the leader tells other actors to play the animation.
	if (const FContextualAnimSceneBinding* Leader = Bindings.GetSyncLeader())
	{
		if (Leader->GetActor() != GetOwner())
		{
			if (UContextualAnimSceneActorComponent* Comp = Leader->GetSceneActorComponent())
			{
				return Comp->TransitionContextualAnimScene(SectionName, AnimSetIdx, ExternalWarpTargets);
			}
		}
	}

	if (const FContextualAnimSceneBinding* OwnerBinding = Bindings.FindBindingByActor(GetOwner()))
	{
		const int32 SectionIdx = Bindings.GetSceneAsset()->GetSectionIndex(SectionName);
		if (SectionIdx != INDEX_NONE)
		{
			UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::TransitionTo Actor: %ls SectionName: %ls",
				*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *SectionName.ToString());

			// Calculate WarpPoints
			TArray<FContextualAnimWarpPoint> WarpPoints;
			CalculateWarpPointsForBindings(Bindings, SectionIdx, AnimSetIdx, WarpPoints);

			HandleTransitionEveryone(SectionIdx, AnimSetIdx, WarpPoints, ExternalWarpTargets);

			RepTransitionData.Id = Bindings.GetID();
			RepTransitionData.SectionIdx = SectionIdx;
			RepTransitionData.AnimSetIdx = AnimSetIdx;
			RepTransitionData.bStopEveryone = false;
			RepTransitionData.WarpPoints = MoveTemp(WarpPoints);
			RepTransitionData.ExternalWarpTargets = ExternalWarpTargets;
			RepTransitionData.IncrementRepCounter();
			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionData, this);
			GetOwner()->ForceNetUpdate();

			return true;
		}
	}

	return false;
}

bool UContextualAnimSceneActorComponent::TransitionContextualAnimScene(FName SectionName, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::TransitionContextualAnimScene Invalid Bindings", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
		return false;
	}

	const int32 SectionIdx = Bindings.GetSceneAsset()->GetSectionIndex(SectionName);
	if (SectionIdx == INDEX_NONE)
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::TransitionContextualAnimScene. Invalid SectionName. Actor: %ls SectionName: %ls",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *SectionName.ToString());

		return false;
	}

	const int32 AnimSetIdx = Bindings.FindAnimSetForTransitionTo(SectionIdx);
	if (AnimSetIdx == INDEX_NONE)
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::TransitionContextualAnimScene. Can't find AnimSet. Actor: %ls SectionName: %ls",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *SectionName.ToString());

		return false;
	}

	return TransitionContextualAnimScene(SectionName, AnimSetIdx, ExternalWarpTargets);
}

void UContextualAnimSceneActorComponent::HandleTransitionEveryone(int32 NewSectionIdx, int32 NewAnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	// Update Bindings internal data and play new animation for the leader first
	// Note that for now we always transition to the first set in the section. We could run selection criteria here too but keeping it simple for now
	HandleTransitionSelf(NewSectionIdx, NewAnimSetIdx, WarpPoints, ExternalWarpTargets);

	// And now the same for everyone else
	for (const FContextualAnimSceneBinding& Binding : Bindings)
	{
		if (Binding.GetActor() != GetOwner())
		{
			if(UContextualAnimSceneActorComponent* Comp = Binding.GetSceneActorComponent())
			{
				Comp->HandleTransitionSelf(NewSectionIdx, NewAnimSetIdx, WarpPoints, ExternalWarpTargets);
			}
		}
	}
}

void UContextualAnimSceneActorComponent::HandleTransitionSelf(int32 NewSectionIdx, int32 NewAnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::HandleTransitionSelf Invalid Bindings. Actor: %ls NewSectionIdx: %d NewAnimSetIdx: %d",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), NewSectionIdx, NewAnimSetIdx);
		return;
	}
	
	// Update bindings internal data so it points to the new section and new anim set
	Bindings.TransitionTo(NewSectionIdx, NewAnimSetIdx);

	// Play animation
	const FContextualAnimSceneBinding& Binding = *Bindings.FindBindingByActor(GetOwner());
	UAnimSequenceBase* AnimationSequence = Binding.GetAnimation();
	PlayAnimation_Internal(AnimationSequence, 0.f, Bindings.ShouldBindingSyncAnimation(Binding));

	AddOrUpdateWarpTargets(NewSectionIdx, NewAnimSetIdx, WarpPoints, ExternalWarpTargets);

	const FContextualAnimTrack& AnimTrack = Bindings.GetAnimTrackFromBinding(Binding);
	if (AnimTrack.bChangeMovementMode)
	{
		if (UCharacterMovementComponent* MovementComp = Binding.GetCharacterMovementComponent())
		{
			if (MovementComp->MovementMode != AnimTrack.MovementMode)
			{
				MovementComp->SetMovementMode(AnimTrack.MovementMode);
			}
		}
	}

	OnTransitionScene(Binding, NewSectionIdx, NewAnimSetIdx);
}

void UContextualAnimSceneActorComponent::OnTransitionScene(const FContextualAnimSceneBinding& Binding, int32 NewSectionIdx, int32 NewAnimSetIdx)
{
	// For derived classes to override.
}

bool UContextualAnimSceneActorComponent::TransitionSingleActor(int32 SectionIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::TransitionSingleActor Invalid Bindings", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
		return false;
	}

	const int32 AnimSetIdx = Bindings.FindAnimSetForTransitionTo(SectionIdx);
	if (AnimSetIdx == INDEX_NONE)
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::TransitionSingleActor. Can't find AnimSet. Actor: %ls SectionIdx: %d",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), SectionIdx);

		return false;
	}

	return TransitionSingleActor(SectionIdx, AnimSetIdx, ExternalWarpTargets);
}

bool UContextualAnimSceneActorComponent::TransitionSingleActor(int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	if (!GetOwner()->HasAuthority())
	{
		return false;
	}

	if (FContextualAnimSceneBinding* OwnerBinding = Bindings.FindBindingByActor(GetOwner()))
	{
		if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
		{
			const FContextualAnimTrack* AnimTrack = Asset->GetAnimTrack(SectionIdx, AnimSetIdx, Bindings.GetRoleFromBinding(*OwnerBinding));
			if (AnimTrack && AnimTrack->Animation)
			{
			
				UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::TransitionSingleActor Actor: %ls SectionIdx: %d AnimSetIdx: %d",
				*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), SectionIdx, AnimSetIdx);

				OwnerBinding->SetAnimTrack(*AnimTrack);
				Bindings.UpdateBindingOverridesIfNeeded();

				// Calculate WarpPoints
				TArray<FContextualAnimWarpPoint> WarpPoints;
				CalculateWarpPointsForBindings(Bindings, SectionIdx, AnimSetIdx, WarpPoints);

				UAnimSequenceBase* AnimationSequence = OwnerBinding->GetAnimation();
				PlayAnimation_Internal(AnimationSequence, 0.f, false);

				AddOrUpdateWarpTargets(SectionIdx, AnimSetIdx, WarpPoints, ExternalWarpTargets);

				if (AnimTrack->bChangeMovementMode)
				{
					if (UCharacterMovementComponent* MovementComp = OwnerBinding->GetCharacterMovementComponent())
					{
						if (MovementComp->MovementMode != AnimTrack->MovementMode)
						{
							MovementComp->SetMovementMode(AnimTrack->MovementMode);
						}
					}
				}

				OnTransitionSingleActor(*OwnerBinding, SectionIdx, AnimSetIdx);

				RepTransitionSingleActorData.Id = Bindings.GetID();
				RepTransitionSingleActorData.SectionIdx = SectionIdx;
				RepTransitionSingleActorData.AnimSetIdx = AnimSetIdx;
				RepTransitionSingleActorData.bStopEveryone = false;
				RepTransitionSingleActorData.WarpPoints = MoveTemp(WarpPoints);
				RepTransitionSingleActorData.ExternalWarpTargets = ExternalWarpTargets;
				RepTransitionSingleActorData.IncrementRepCounter();
				MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionSingleActorData, this);
				GetOwner()->ForceNetUpdate();

				return true;
			}
		}
	}

	return false;
}

void UContextualAnimSceneActorComponent::OnTransitionSingleActor(const FContextualAnimSceneBinding& Binding, int32 NewSectionIdx, int32 NewAnimSetIdx)
{
	// For derived classes to override.
}

void UContextualAnimSceneActorComponent::OnRep_RepTransitionSingleActor()
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnRep_RepTransitionSingleActor Owner: %ls Id: %d RepCounter: %d SectionIdx: %d AnimSetIdx: %d Current Bindings ID: %d",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), 
		RepTransitionSingleActorData.Id, RepTransitionSingleActorData.RepCounter, RepTransitionSingleActorData.SectionIdx, RepTransitionSingleActorData.AnimSetIdx, Bindings.IsValid() ? Bindings.GetID() : -1);

	if (!RepTransitionSingleActorData.IsValid())
	{
		return;
	}

	if (FContextualAnimSceneBinding* OwnerBinding = Bindings.FindBindingByActor(GetOwner()))
	{
		if (RepTransitionSingleActorData.SectionIdx != MAX_uint8 && RepTransitionSingleActorData.AnimSetIdx != MAX_uint8)
		{
			if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
			{
				const FContextualAnimTrack* AnimTrack = Asset->GetAnimTrack(RepTransitionSingleActorData.SectionIdx, RepTransitionSingleActorData.AnimSetIdx, Bindings.GetRoleFromBinding(*OwnerBinding));
				if (AnimTrack)
				{
					OwnerBinding->SetAnimTrack(*AnimTrack);
					Bindings.UpdateBindingOverridesIfNeeded();

					UAnimSequenceBase* TrackAnimationSequence = OwnerBinding->GetAnimation();
					PlayAnimation_Internal(TrackAnimationSequence, 0.f, false);

					AddOrUpdateWarpTargets(RepTransitionSingleActorData.SectionIdx, RepTransitionSingleActorData.AnimSetIdx, RepTransitionSingleActorData.WarpPoints, RepTransitionSingleActorData.ExternalWarpTargets);

					if (AnimTrack->bChangeMovementMode)
					{
						if (UCharacterMovementComponent* MovementComp = OwnerBinding->GetCharacterMovementComponent())
						{
							if (MovementComp->MovementMode != AnimTrack->MovementMode)
							{
								MovementComp->SetMovementMode(AnimTrack->MovementMode);
							}
						}
					}

					OnTransitionSingleActor(*OwnerBinding, RepTransitionSingleActorData.SectionIdx, RepTransitionSingleActorData.AnimSetIdx);
				}
			}
		}
		else
		{
			// RepTransitionSingleActorData with invalid indices is replicated when the animation ends
			// In this case we don't want to tell everyone else to also leave the scene since there is very common for the initiator, 
			// specially if is player character, to end the animation earlier for responsiveness
			// It is more likely this will do nothing since we listen to montage end also on Simulated Proxies to 'predict' the end of the interaction.
			if (RepTransitionSingleActorData.Id == Bindings.GetID())
			{
				// Ensure that all other actors stop their animations if requested.
				if (RepTransitionSingleActorData.bStopEveryone)
				{
					// @TODO: We copy bindings other we would be iterating over an array that would change in size due to LeveScene() updates the other's actor bindings.
					// This should go away once we add the concept of a manager and keep bindings in a single place
					FContextualAnimSceneBindings TempBindings = Bindings;

					for (const FContextualAnimSceneBinding& OtherBinding : TempBindings)
					{
						if (OtherBinding.GetActor() != GetOwner())
						{
							if (UContextualAnimSceneActorComponent* Comp = OtherBinding.GetSceneActorComponent())
							{
								Comp->LeaveScene();
							}
						}
					}
				}

				LeaveScene();
			}
		}
	}
}

bool UContextualAnimSceneActorComponent::StartContextualAnimScene(const FContextualAnimSceneBindings& InBindings)
{
	return StartContextualAnimScene(InBindings, {});
}
	
bool UContextualAnimSceneActorComponent::LateJoinContextualAnimScene(AActor* Actor, FName Role)
{
	return LateJoinContextualAnimScene(Actor, Role, {});
}
	
bool UContextualAnimSceneActorComponent::TransitionContextualAnimScene(FName SectionName)
{
	return TransitionContextualAnimScene(SectionName, {});
}

bool UContextualAnimSceneActorComponent::TransitionContextualAnimSceneToSpecificSet(FName SectionName, int32 AnimSetIdx)
{
	return TransitionContextualAnimScene(SectionName, AnimSetIdx, {});
}
	
bool UContextualAnimSceneActorComponent::TransitionSingleActor(int32 SectionIdx, int32 AnimSetIdx)
{
	return TransitionSingleActor(SectionIdx, AnimSetIdx, {});
}

float UContextualAnimSceneActorComponent::FindIdealStartTime(const FContextualAnimSceneBindings& InBindings, const FContextualAnimSceneBinding& Binding, const TArray<FContextualAnimWarpPoint>& WarpPoints) const
{
	float IdealStartTime = 0.f;

	const UContextualAnimSceneAsset* Asset = InBindings.GetSceneAsset();
	if (Asset && Binding.GetAnimation() && WarpPoints.Num() > 0)
	{
		// Get the end time for the first warp section in the animation
		// This is our hard upper bound since we need to give some chance to motion warping to run
		float WarpStartTime, WarpEndTime;
		FName WarpTargetName = UContextualAnimUtilities::GetStartAndEndTimeForWarpSection(Binding.GetAnimation(), 0, WarpStartTime, WarpEndTime);

		// Find the warp point that matches the warp target name from the animation
		const FContextualAnimWarpPoint* WarpPoint = WarpPoints.FindByPredicate([WarpTargetName](const FContextualAnimWarpPoint& Item) { return Item.Name == WarpTargetName; });
		if (WarpPoint == nullptr)
		{
			UE_LOGF(LogContextualAnim, Verbose, "FindIdealStartTime: No warp point found for WarpTargetName: %ls Actor: %ls Animation: %ls",
				*WarpTargetName.ToString(), *GetNameSafe(GetOwner()), *GetNameSafe(Binding.GetAnimation()));
			return IdealStartTime;
		}

		// Early out if there is no enough room for motion warping to run
		const float MaxSearchTime = WarpEndTime - Asset->GetFindIdealStartTimeMargin();
		if (MaxSearchTime <= 0.f)
		{
			return IdealStartTime;
		}

		// Calculate where the animation expects the character to be at T=0 and at sync time, both relative to the warp point
		const FVector AnimStartLocation = Asset->GetAlignmentTransform(Binding, InBindings, WarpTargetName, 0.f).GetLocation();
		const FVector AnimSyncLocation = Asset->GetAlignmentTransform(Binding, InBindings, WarpTargetName, WarpEndTime).GetLocation();

		// Actor's actual position relative to the warp point
		const FVector ActorLocationRelative = GetOwner()->GetActorTransform().GetRelativeTransform(WarpPoint->Transform).GetLocation();

		// Distance from the start location to the warp target in the animation
		const float PerfectDistSq = FVector::DistSquared2D(AnimStartLocation, AnimSyncLocation);

		// Distance from where the actor actually is in game to the warp target
		const float ActualDistSq = FVector::DistSquared2D(ActorLocationRelative, AnimSyncLocation);

		// Actor is closer than the full authored approach distance. 
		// Find the time in the animation where the remaining distance to the target matches the actor's actual distance.
		if (ActualDistSq < PerfectDistSq)
		{
			const float SampleInterval = 1.f / 30.f;

			for (float Time = 0.f; Time <= MaxSearchTime; Time += SampleInterval)
			{
				const FVector FrameLocation = Asset->GetAlignmentTransform(Binding, InBindings, WarpPoint->Name, Time).GetLocation();
				const float FrameDistSq = FVector::DistSquared2D(FrameLocation, AnimSyncLocation);

				if (FrameDistSq <= ActualDistSq)
				{
					IdealStartTime = Time;
					break;
				}
			}
		}

		UE_LOGF(LogContextualAnim, Verbose, "IdealStartTime: %.3f (MaxSearch: %.3f) ActualDist: %.1f PerfectDist: %.1f Actor: %ls Animation: %ls AnimStartLoc: %ls AnimSyncLoc: %ls ActorLocRelative: %ls",
			IdealStartTime, MaxSearchTime, FMath::Sqrt(ActualDistSq), FMath::Sqrt(PerfectDistSq),
			*GetNameSafe(GetOwner()), *GetNameSafe(Binding.GetAnimation()), *AnimStartLocation.ToString(), *AnimSyncLocation.ToString(), *ActorLocationRelative.ToString());
	}

	return IdealStartTime;
}

bool UContextualAnimSceneActorComponent::StartContextualAnimScene(const FContextualAnimSceneBindings& InBindings, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets)
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::StartContextualAnim Actor: %ls",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()));

	const FContextualAnimSceneBinding* OwnerBinding = InBindings.FindBindingByActor(GetOwner());
	if (ensureAlways(OwnerBinding))
	{
		if (GetOwner()->HasAuthority())
		{
			// @TODO: Warp points are calculated on the server and replicated to everyone to avoid mismatch when they are relative to moving actor(s)
			// It may be better to have them in the Bindings but during late joint the actor that is joining the interaction starts from the first section, 
			// which could be different from the section the other actors are playing.
			// We should reconsider all that in the future, maybe moving SectionIdx and AnimSetIdx out of the bindings if we want to support that case long term.
			TArray<FContextualAnimWarpPoint> WarpPoints;
			CalculateWarpPointsForBindings(InBindings, OwnerBinding->GetSectionIdx(), OwnerBinding->GetAnimSetIdx(), WarpPoints);

			// Finds ideal start time based on the distance from the actor to the warp target in the animation and the actual distance in game.
			// Note this time is computed from the initiator's position and all other actors use the same value to stay in sync.
			float IdealStartTime = 0.f;
			const UContextualAnimSceneAsset* Asset = InBindings.GetSceneAsset();
			if (Asset && Asset->ShouldFindIdealAnimStartTime())
			{
				IdealStartTime = FindIdealStartTime(InBindings, *OwnerBinding, WarpPoints);
			}

			JoinScene(InBindings, WarpPoints, ExternalWarpTargets, IdealStartTime);

			for (const FContextualAnimSceneBinding& Binding : InBindings)
			{
				if (Binding.GetActor() != GetOwner())
				{
					if (UContextualAnimSceneActorComponent* Comp = Binding.GetSceneActorComponent())
					{
						Comp->JoinScene(InBindings, WarpPoints, ExternalWarpTargets, IdealStartTime);
					}
				}
			}

			RepBindings.Bindings = InBindings;
			RepBindings.WarpPoints = MoveTemp(WarpPoints);
			RepBindings.ExternalWarpTargets = ExternalWarpTargets;
			RepBindings.StartTime = IdealStartTime;
			RepBindings.IncrementRepCounter();

			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepBindings, this);
			GetOwner()->ForceNetUpdate();

			return true;
		}
		else if (GetOwner()->GetLocalRole() == ROLE_AutonomousProxy)
		{
			ServerStartContextualAnimScene(InBindings);
			return true;
		}
	}

	return false;
}

void UContextualAnimSceneActorComponent::ServerStartContextualAnimScene_Implementation(const FContextualAnimSceneBindings& InBindings)
{
	StartContextualAnimScene(InBindings, {});
}

bool UContextualAnimSceneActorComponent::ServerStartContextualAnimScene_Validate(const FContextualAnimSceneBindings& InBindings)
{
	return true;
}

void UContextualAnimSceneActorComponent::EarlyOutContextualAnimScene(bool bStopEveryone)
{
	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(GetOwner()))
	{
		const UAnimInstance* AnimInstance = Binding->GetAnimInstance();
		const UAnimMontage* ActiveMontage = AnimInstance ? AnimInstance->GetCurrentActiveMontage() : nullptr;
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::EarlyOutContextualAnimScene Actor: %ls ActiveMontage: %ls bStopEveryone: %d",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *GetNameSafe(ActiveMontage), bStopEveryone);

		if (Binding->GetAnimation() == ActiveMontage)
		{
			const uint8 BindingsId = Bindings.GetID();

			// Ensure that all other actors stop their animations if requested. We do this first since the calling LeaveScene() clears all the bindings.
			if (bStopEveryone)
			{
				// @TODO: We copy bindings other we would be iterating over an array that would change in size due to LeveScene() updates the other's actor bindings. 
				// This should go away once we add the concept of a manager and keep bindings in a single place
				FContextualAnimSceneBindings TempBindings = Bindings;

				for (const FContextualAnimSceneBinding& OtherBinding : TempBindings)
				{
					if (OtherBinding.GetActor() != GetOwner())
					{
						if (UContextualAnimSceneActorComponent* Comp = OtherBinding.GetSceneActorComponent())
						{
							Comp->LeaveScene();
						}
					}
				}
			}

			// Stop animation.
			LeaveScene();

			// If we are on the server, rep the event to stop animation on simulated proxies
			if (GetOwner()->HasAuthority())
			{
				RepTransitionSingleActorData.Id = BindingsId;
				RepTransitionSingleActorData.SectionIdx = MAX_uint8;
				RepTransitionSingleActorData.AnimSetIdx = MAX_uint8;
				RepTransitionSingleActorData.bStopEveryone = bStopEveryone;
				RepTransitionSingleActorData.WarpPoints.Reset();
				RepTransitionSingleActorData.ExternalWarpTargets.Reset();
				RepTransitionSingleActorData.IncrementRepCounter();

				RepLateJoinData.Reset();
				RepTransitionData.Reset();
				RepBindings.Reset();

				MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionSingleActorData, this);
				MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepLateJoinData, this);
				MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionData, this);
				MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepBindings, this);

				GetOwner()->ForceNetUpdate();
			}
			// If local player, tell the server to stop the animation too
			else if (GetOwner()->GetLocalRole() == ROLE_AutonomousProxy)
			{
				ServerEarlyOutContextualAnimScene(bStopEveryone);
			}
		}
	}
}

void UContextualAnimSceneActorComponent::ServerEarlyOutContextualAnimScene_Implementation(bool bStopEveryone)
{
	EarlyOutContextualAnimScene(bStopEveryone);
}

bool UContextualAnimSceneActorComponent::ServerEarlyOutContextualAnimScene_Validate(bool bStopEveryone)
{
	return true;
}

void UContextualAnimSceneActorComponent::OnRep_TransitionData()
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnRep_TransitionData Actor: %ls SectionIdx: %d AnimsetIdx: %d RepCounter: %d",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), RepTransitionData.SectionIdx, RepTransitionData.AnimSetIdx, RepTransitionData.RepCounter);

	if (!RepTransitionData.IsValid())
	{
		return;
	}

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "%-21ls UContextualAnimSceneActorComponent::OnRep_TransitionData Actor: %ls Current bindings INVALID",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()));

		return;
	}

	HandleTransitionEveryone(RepTransitionData.SectionIdx, RepTransitionData.AnimSetIdx, RepTransitionData.WarpPoints, RepTransitionData.ExternalWarpTargets);
}

void UContextualAnimSceneActorComponent::OnRep_Bindings()
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnRep_Bindings Actor: %ls Rep Bindings Id: %d RepCounter: %d Num: %d Current Bindings Id: %d Num: %d",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), RepBindings.Bindings.GetID(), RepBindings.RepCounter, RepBindings.Bindings.Num(), Bindings.GetID(), Bindings.Num());

	if (!RepBindings.IsValid())
	{
		return;
	}

	// The owner of this component started an interaction on the server
	if (RepBindings.Bindings.IsValid())
	{
		// Locally Update the Binding Overrides, this may generate IK and ALignment Tracks if Needed
		RepBindings.Bindings.UpdateBindingOverridesIfNeeded();

		const FContextualAnimSceneBinding* OwnerBinding = RepBindings.Bindings.FindBindingByActor(GetOwner());
		if (ensureAlways(OwnerBinding))
		{
			// Join the scene (start playing animation, etc.)
			JoinScene(RepBindings.Bindings, RepBindings.WarpPoints, RepBindings.ExternalWarpTargets, RepBindings.StartTime);

			// RepBindings is only replicated from the initiator of the action.
			// So now we have to tell everyone else involved in the interaction to join us
			// @TODO: For now this assumes that all the actors will start playing the animation at the same time. 
			// We will expand this in the future to allow 'late' join
			for (const FContextualAnimSceneBinding& Binding : RepBindings.Bindings)
			{
				if (Binding.GetActor() != GetOwner())
				{
					UContextualAnimSceneActorComponent* Comp = Binding.GetSceneActorComponent();

					if (ensureMsgf(Comp, TEXT("%-21s UContextualAnimSceneActorComponent::OnRep_Bindings Actor: %s Current UContextualAnimSceneActorComponent INVALID"),
						*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(Binding.GetActor())))
					{
						Comp->JoinScene(RepBindings.Bindings, RepBindings.WarpPoints, RepBindings.ExternalWarpTargets, RepBindings.StartTime);
					}
				}
			}
		}
	}
}

void UContextualAnimSceneActorComponent::SetIgnoreCollisionWithOtherActors(bool bValue) const
{
	const AActor* OwnerActor = GetOwner();

	for (const FContextualAnimSceneBinding& Binding : Bindings)
	{
		AActor* OtherActor = Binding.GetActor();
		if (OtherActor && OtherActor != OwnerActor)
		{
			SetIgnoreCollisionWithActor(*OtherActor, bValue);
		}
	}
}

void UContextualAnimSceneActorComponent::SetIgnoreCollisionWithActor(AActor& Actor, bool bValue) const
{
	if (UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetOwner()->GetRootComponent()))
	{
		RootPrimitiveComponent->IgnoreActorWhenMoving(&Actor, bValue);

		// MoveIgnoreActors is an array of hard references, so handle this actor receiving EndPlay (e.g. when it streams out)
		//  so we explicitly remove its reference, which unblocks it from being cleaned up by garbage collection
		if (bValue)
		{
			Actor.OnEndPlay.AddUniqueDynamic(const_cast<UContextualAnimSceneActorComponent*>(this), &UContextualAnimSceneActorComponent::OnIgnoredActorEndPlay);
		}
		else
		{
			Actor.OnEndPlay.RemoveDynamic(this, &UContextualAnimSceneActorComponent::OnIgnoredActorEndPlay);
		}
	}
}

void UContextualAnimSceneActorComponent::OnIgnoredActorEndPlay(AActor* InEndPlayActor, EEndPlayReason::Type EndPlayReason)
{
	AActor* OwnerActor = GetOwner();
	if (!OwnerActor)
	{
		return;
	}

	UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(OwnerActor->GetRootComponent());
	if (!RootPrimitiveComponent)
	{
		return;
	}

	// If we were ignoring an Actor and it was removed from Play, remove it from our MoveIgnoreActors list immediately 
	//  so that this hard reference to it doesn't block garbage collection, causing leaks
	RootPrimitiveComponent->IgnoreActorWhenMoving(InEndPlayActor, false);
}

void UContextualAnimSceneActorComponent::SetCollisionState(const FContextualAnimSceneBinding& Binding)
{
	if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
	{
		const TArray<FContextualAnimCollisionBehaviorParams>& CollisionBehaviorParams = Asset->GetCollisionBehaviors();
		for (const FContextualAnimCollisionBehaviorParams& CollisionBehaviorParam : CollisionBehaviorParams)
		{
			if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreActorWhenMoving)
			{
				SetIgnoreCollisionWithOtherActors(true);
			}
			else if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreChannels)
			{
				if (UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetOwner()->GetRootComponent()))
				{
					const TArray<TEnumAsByte<ECollisionChannel>>& ChannelsToIgnore = CollisionBehaviorParam.GetCollisionChannelsToIgnoreForRole(Bindings.GetRoleFromBinding(Binding));
					if (ChannelsToIgnore.Num() > 0)
					{
						CharacterPropertiesBackup.CollisionResponses.Reset(ChannelsToIgnore.Num());
						for (ECollisionChannel Channel : ChannelsToIgnore)
						{
							ECollisionResponse Response = RootPrimitiveComponent->GetCollisionResponseToChannel(Channel);
							CharacterPropertiesBackup.CollisionResponses.Add(MakeTuple(Channel, Response));

							RootPrimitiveComponent->SetCollisionResponseToChannel(Channel, ECR_Ignore);
						}
					}
				}
			}
			else if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::CollisionProfile)
			{
				if (UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetOwner()->GetRootComponent()))
				{
					CharacterPropertiesBackup.CollisionProfileResponses = RootPrimitiveComponent->GetCollisionResponseToChannels();
					RootPrimitiveComponent->SetCollisionProfileName(CollisionBehaviorParam.GetCollisionProfileNameForRole(Bindings.GetRoleFromBinding(Binding)).Name);
				}
			}
		}
	}
}

void UContextualAnimSceneActorComponent::RestoreCollisionState(const FContextualAnimSceneBinding& Binding)
{
	if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
	{
		const TArray<FContextualAnimCollisionBehaviorParams>& CollisionBehaviorParams = Asset->GetCollisionBehaviors();
		for (const FContextualAnimCollisionBehaviorParams& CollisionBehaviorParam : CollisionBehaviorParams)
		{
			if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreActorWhenMoving)
			{
				SetIgnoreCollisionWithOtherActors(false);
			}
			else if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreChannels)
			{
				if (UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetOwner()->GetRootComponent()))
				{
					for (const TTuple<ECollisionChannel, ECollisionResponse>& Response : CharacterPropertiesBackup.CollisionResponses)
					{
						RootPrimitiveComponent->SetCollisionResponseToChannel(Response.Get<0>(), Response.Get<1>());
					}

					CharacterPropertiesBackup.CollisionResponses.Reset();
				}
			}
			else if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::CollisionProfile)
			{
				if (UPrimitiveComponent* RootPrimitiveComponent = Cast<UPrimitiveComponent>(GetOwner()->GetRootComponent()))
				{
					if (CharacterPropertiesBackup.CollisionProfileResponses.IsSet())
					{
						RootPrimitiveComponent->SetCollisionResponseToChannels(CharacterPropertiesBackup.CollisionProfileResponses.GetValue());
						CharacterPropertiesBackup.CollisionProfileResponses.Reset();
					}
				}
			}
		}
	}
}

void UContextualAnimSceneActorComponent::OnJoinedScene(const FContextualAnimSceneBindings& InBindings)
{
	// This function will be removed
}

void UContextualAnimSceneActorComponent::OnLeftScene()
{
	// This function will be removed
}

void UContextualAnimSceneActorComponent::JoinScene(const FContextualAnimSceneBindings& InBindings, const TArray<FContextualAnimWarpPoint> WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets, float StartTime)
{
	if (Bindings.IsValid())
	{
		LeaveScene();
	}

	if (const FContextualAnimSceneBinding* Binding = InBindings.FindBindingByActor(GetOwner()))
	{
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::JoinScene Actor: %ls Role: %ls InBindings Id: %d Asset: %ls",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *InBindings.GetRoleFromBinding(*Binding).ToString(), InBindings.GetID(), *GetNameSafe(InBindings.GetSceneAsset()));

		AnimsPlayed.Reset();

		Bindings = InBindings;

		UAnimSequenceBase* AnimationSequence = Binding->GetAnimation();
		PlayAnimation_Internal(AnimationSequence, StartTime, Bindings.ShouldBindingSyncAnimation(*Binding));

		AddOrUpdateWarpTargets(Binding->GetSectionIdx(), Binding->GetAnimSetIdx(), WarpPoints, ExternalWarpTargets);

		SetCollisionState(*Binding);

		const FContextualAnimTrack& AnimTrack = Bindings.GetAnimTrackFromBinding(*Binding);
		SetMovementState(*Binding, AnimTrack);

		OnJoinScene(*Binding);

		OnJoinedSceneDelegate.Broadcast(this);
	}
}

void UContextualAnimSceneActorComponent::OnJoinScene(const FContextualAnimSceneBinding& Binding)
{
	// For derived classes to override.
}

void UContextualAnimSceneActorComponent::LeaveScene()
{
	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(GetOwner()))
	{
		OnPreLeaveScene(*Binding);
		
		UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::LeaveScene Actor: %ls Role: %ls Current Bindings Id: %d Asset: %ls",
			*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *Bindings.GetRoleFromBinding(*Binding).ToString(),
			Bindings.GetID(), *GetNameSafe(Bindings.GetSceneAsset()));

		if (OwnerAnimInstance.IsValid())
		{
			OwnerAnimInstance->OnMontageBlendingOut.RemoveDynamic(this, &UContextualAnimSceneActorComponent::OnMontageBlendingOut);
			OwnerAnimInstance->OnPlayMontageNotifyBegin.RemoveDynamic(this, &UContextualAnimSceneActorComponent::OnPlayMontageNotifyBegin);

			//@TODO: Add support for dynamic montage
			const UAnimMontage* AnimMontage = OwnerAnimInstance->GetCurrentActiveMontage();
			if (AnimMontage)
			{
				UE_LOGF(LogContextualAnim, VeryVerbose, "\t\t Stopping animation (%ls) from LeaveScene", *GetNameSafe(AnimMontage));
				OwnerAnimInstance->Montage_Stop(AnimMontage->GetDefaultBlendOutTime());
			}

			OwnerAnimInstance.Reset();
		}

		// Stop listening to TickPose if we were
		USkeletalMeshComponent* SkelMeshComp = Binding->GetSkeletalMeshComponent();
		if (SkelMeshComp && SkelMeshComp->OnTickPose.IsBoundToObject(this))
		{
			SkelMeshComp->OnTickPose.RemoveAll(this);
		}
		
		RestoreCollisionState(*Binding);

		RestoreMovementState(*Binding);

		if (WarpTargetNamesCache.Num() > 0)
		{
			if (UMotionWarpingComponent* MotionWarpComp = Binding->GetMotionWarpingComponent())
			{
				MotionWarpComp->RemoveWarpTargets(WarpTargetNamesCache);
			}

			WarpTargetNamesCache.Reset();
		}

		IKTargets.Reset();

		// Notify the other actors in the interaction
		// @TODO: This should be refactored so only the leader of the interaction maintains the full bindings
		for (const FContextualAnimSceneBinding& OtherBinding : Bindings)
		{
			AActor* OwnerActor = GetOwner();
			if (OtherBinding.GetActor() != OwnerActor)
			{
				if (UContextualAnimSceneActorComponent* Comp = OtherBinding.GetSceneActorComponent())
				{
					Comp->OtherActorLeftScene(*OwnerActor);
				}
			}
		}

		OnLeaveScene(*Binding);

		AnimsPlayed.Reset();

		Bindings.Reset();

		OnLeftSceneDelegate.Broadcast(this);
	}
}

void UContextualAnimSceneActorComponent::OtherActorLeftScene(AActor& Actor)
{
	if (Bindings.IsValid())
	{
		if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
		{
			const TArray<FContextualAnimCollisionBehaviorParams>& CollisionBehaviorParams = Asset->GetCollisionBehaviors();
			for (const FContextualAnimCollisionBehaviorParams& CollisionBehaviorParam : CollisionBehaviorParams)
			{
				if (CollisionBehaviorParam.CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreActorWhenMoving)
				{
					SetIgnoreCollisionWithActor(Actor, false);
				}
			}

			Bindings.RemoveActor(Actor);
		}
	}
}

void UContextualAnimSceneActorComponent::OnLeaveScene(const FContextualAnimSceneBinding& Binding)
{
	// For derived classes to override.
}

void UContextualAnimSceneActorComponent::SetMovementState(const FContextualAnimSceneBinding& Binding, const FContextualAnimTrack& AnimTrack)
{
	if (UCharacterMovementComponent* MovementComp = Binding.GetCharacterMovementComponent())
	{
		// Save movement state before the interaction starts so we can restore it when it ends
		CharacterPropertiesBackup.bSimulatePhysics = MovementComp->UpdatedPrimitive && MovementComp->UpdatedPrimitive->IsSimulatingPhysics();

		// Disable movement correction if needed
		if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
		{
			if (Asset->ShouldIgnoreClientMovementErrorChecksAndCorrection())
			{
				CharacterPropertiesBackup.bIgnoreClientMovementErrorChecksAndCorrection = MovementComp->bIgnoreClientMovementErrorChecksAndCorrection;
				MovementComp->bIgnoreClientMovementErrorChecksAndCorrection = true;
			}

			if (Asset->ShouldDisableMovementReplicationForSimulatedProxy())
			{
				AActor* OwnerActor = GetOwner();
				if (OwnerActor && OwnerActor->HasAuthority() && OwnerActor->GetRemoteRole() == ROLE_SimulatedProxy)
				{
					CharacterPropertiesBackup.bReplicatingMovement = OwnerActor->IsReplicatingMovement();
					OwnerActor->SetReplicateMovement(false);
				}
			}
		}

		if (AnimTrack.bControlCharacterRotation)
		{
			CharacterPropertiesBackup.RotationProperties = FCharacterRotationProperties();
			CharacterPropertiesBackup.RotationProperties->bAllowPhysicsRotationDuringAnimRootMotion = MovementComp->bAllowPhysicsRotationDuringAnimRootMotion;
			CharacterPropertiesBackup.RotationProperties->bUseControllerDesiredRotation = MovementComp->bUseControllerDesiredRotation;
			CharacterPropertiesBackup.RotationProperties->bOrientRotationToMovement = MovementComp->bOrientRotationToMovement;
			
			// Prevent physics rotation. During the interaction we want to be fully root motion driven
			MovementComp->bAllowPhysicsRotationDuringAnimRootMotion = false;
			MovementComp->bUseControllerDesiredRotation = false;
			MovementComp->bOrientRotationToMovement = false;
		}
		else
		{
			CharacterPropertiesBackup.RotationProperties.Reset();
		}

		if (UPrimitiveComponent* const UpdatedPrimitive = MovementComp->UpdatedPrimitive)
		{
			UpdatedPrimitive->SetSimulatePhysics(false);
		}

		if (AnimTrack.bChangeMovementMode)
		{
			CharacterPropertiesBackup.MovementMode = MovementComp->MovementMode;
			CharacterPropertiesBackup.CustomMovementMode = MovementComp->CustomMovementMode;
			if (MovementComp->MovementMode != AnimTrack.MovementMode)
			{
				MovementComp->SetMovementMode(AnimTrack.MovementMode, AnimTrack.CustomMovementMode);
			}
		}
		else
		{
			CharacterPropertiesBackup.MovementMode.Reset();
			CharacterPropertiesBackup.CustomMovementMode = 0;
		}
	}
}

void UContextualAnimSceneActorComponent::RestoreMovementState(const FContextualAnimSceneBinding& Binding)
{
	// Restore movement state
	if (UCharacterMovementComponent* MovementComp = Binding.GetCharacterMovementComponent())
	{
		// Restore movement correction if needed
		if (const UContextualAnimSceneAsset* Asset = Bindings.GetSceneAsset())
		{
			if (Asset->ShouldIgnoreClientMovementErrorChecksAndCorrection())
			{
				MovementComp->bIgnoreClientMovementErrorChecksAndCorrection = CharacterPropertiesBackup.bIgnoreClientMovementErrorChecksAndCorrection;
			}

			if (Asset->ShouldDisableMovementReplicationForSimulatedProxy())
			{
				AActor* OwnerActor = GetOwner();
				if (OwnerActor && OwnerActor->HasAuthority() && OwnerActor->GetRemoteRole() == ROLE_SimulatedProxy)
				{
					OwnerActor->SetReplicateMovement(CharacterPropertiesBackup.bReplicatingMovement);
				}
			}
		}

		if (CharacterPropertiesBackup.RotationProperties.IsSet())
		{
			MovementComp->bAllowPhysicsRotationDuringAnimRootMotion = CharacterPropertiesBackup.RotationProperties->bAllowPhysicsRotationDuringAnimRootMotion;
			MovementComp->bUseControllerDesiredRotation = CharacterPropertiesBackup.RotationProperties->bUseControllerDesiredRotation;
			MovementComp->bOrientRotationToMovement = CharacterPropertiesBackup.RotationProperties->bOrientRotationToMovement;
		}
		
		if (CharacterPropertiesBackup.MovementMode.IsSet())
		{
			if (
				(CharacterPropertiesBackup.MovementMode == MOVE_Walking || CharacterPropertiesBackup.MovementMode == MOVE_NavWalking) &&
				(MovementComp->GetGroundMovementMode() == MOVE_Walking || MovementComp->GetGroundMovementMode() == MOVE_NavWalking))
			{
				// This is mostly for AIs, we want to make sure to not override their ground movement mode if it changes while the anim is playing
				// @TODO: We should have a more robust mechanism around this to prevent the same problem with any of the movement properties that change during the interaction.
				MovementComp->SetMovementMode(MovementComp->GetGroundMovementMode(), CharacterPropertiesBackup.CustomMovementMode);
			}
			else if (*CharacterPropertiesBackup.MovementMode)
			{
				MovementComp->SetMovementMode(*CharacterPropertiesBackup.MovementMode, CharacterPropertiesBackup.CustomMovementMode);
			}
		}
		
		if (UPrimitiveComponent* const UpdatedPrimitive = MovementComp->UpdatedPrimitive)
		{
			UpdatedPrimitive->SetSimulatePhysics(CharacterPropertiesBackup.bSimulatePhysics);
		}
	}
}

bool UContextualAnimSceneActorComponent::CanLeaveScene(const FContextualAnimSceneBinding& Binding)
{
	return true;
}

void UContextualAnimSceneActorComponent::OnPreLeaveScene(const FContextualAnimSceneBinding& Binding)
{
	// For derived classes to override.
}

void UContextualAnimSceneActorComponent::OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted)
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnMontageBlendingOut Actor: %ls Montage: %ls bInterrupted: %d",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *GetNameSafe(Montage), bInterrupted);

	if (const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(GetOwner()))
	{
		//We fire this delegate only when an anim that's part of the state tree plays so we can allow player to play montages during SO interactions (if the SO allows it).
		int32 NumAnimsRemoved = AnimsPlayed.RemoveSingleSwap(Montage);
		if (NumAnimsRemoved > 0)
		{
			OnMontageBlendingOutDelegate.Broadcast(this, Montage, bInterrupted);
		}

		if (AnimsPlayed.Num() > 0)
		{
			UE_LOGF(LogContextualAnim, Verbose, "%-21ls \tUContextualAnimSceneActorComponent::OnMontageBlendingOut AnimsPlayed Num: %d", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), AnimsPlayed.Num());
			return;
		}

		if (!CanLeaveScene(*Binding))
		{
			UE_LOGF(LogContextualAnim, Verbose, "%-21ls \tUContextualAnimSceneActorComponent::OnMontageBlendingOut CanLeaveScene FALSE", *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()));
			return;
		}

		const uint8 BindingsId = Bindings.GetID();

		// Stop animation, restore state etc.
		LeaveScene();

		if (GetOwner()->HasAuthority())
		{
			RepTransitionSingleActorData.Id = BindingsId;
			RepTransitionSingleActorData.SectionIdx = MAX_uint8;
			RepTransitionSingleActorData.AnimSetIdx = MAX_uint8;
			RepTransitionSingleActorData.bStopEveryone = false;
			RepTransitionSingleActorData.WarpPoints.Reset();
			RepTransitionSingleActorData.ExternalWarpTargets.Reset();
			RepTransitionSingleActorData.IncrementRepCounter();

			RepLateJoinData.Reset();
			RepTransitionData.Reset();
			RepBindings.Reset();

			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionSingleActorData, this);
			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepLateJoinData, this);
			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepTransitionData, this);
			MARK_PROPERTY_DIRTY_FROM_NAME(UContextualAnimSceneActorComponent, RepBindings, this);

			GetOwner()->ForceNetUpdate();
		}
	}
}

void UContextualAnimSceneActorComponent::OnPlayMontageNotifyBegin(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload)
{
	UE_LOGF(LogContextualAnim, Verbose, "%-21ls UContextualAnimSceneActorComponent::OnNotifyBeginReceived Actor: %ls Animation: %ls NotifyName: %ls",
		*UEnum::GetValueAsString(TEXT("Engine.ENetRole"), GetOwner()->GetLocalRole()), *GetNameSafe(GetOwner()), *GetNameSafe(BranchingPointNotifyPayload.SequenceAsset), *NotifyName.ToString());

	OnPlayMontageNotifyBeginDelegate.Broadcast(this, NotifyName);

}

void UContextualAnimSceneActorComponent::OnTickPose(class USkinnedMeshComponent* SkinnedMeshComponent, float DeltaTime, bool bNeedsValidRootMotion)
{
	// Fire OnMontageReachedEndDelegate if needed.
	const FContextualAnimSceneBinding* BindingPtr = Bindings.IsValid() ? Bindings.FindBindingByActor(GetOwner()) : nullptr;
	const FAnimMontageInstance* MontageInstance = BindingPtr ? BindingPtr->GetAnimMontageInstance() : nullptr;
	if (MontageInstance && MontageInstance->Montage && MontageInstance->Montage == BindingPtr->GetAnimation())
	{
		const float MontagePos = MontageInstance->GetPosition();
		const FName NextSection = MontageInstance->GetNextSection();

		// EnableAutoBlendOut in the montage is set to false
		if (NextSection == NAME_None && MontageInstance->Montage->bEnableAutoBlendOut == false)
		{
			float StartTime = 0.f;
			float EndTime = 0.f;
			MontageInstance->Montage->GetSectionStartAndEndTime(MontageInstance->Montage->GetSectionIndex(MontageInstance->GetCurrentSection()), StartTime, EndTime);

			// Use LoopCounter in this case as a flag to ensure we fire OnMontageReachedEndDelegate only once
			if (LoopCounter == 0 && FMath::IsNearlyEqual(MontagePos, EndTime, UE_KINDA_SMALL_NUMBER)) 
			{
				OnMontageReachedEndDelegate.Broadcast(this, MontageInstance->Montage, 0);
				LoopCounter++;
			}
		}
		// Section in the montage is set to loop
		else if (NextSection == MontageInstance->GetCurrentSection())
		{
			if (MontagePos < MontageInstance->GetPreviousPosition())
			{
				LoopCounter++;
				OnMontageReachedEndDelegate.Broadcast(this, MontageInstance->Montage, LoopCounter);
			}
		}
	}

	//@TODO: Check for LOD too to prevent this update if the actor is too far away
	if (GetNetMode() != NM_DedicatedServer)
	{
		UpdateIKTargets();
	}
}

void UContextualAnimSceneActorComponent::UpdateIKTargets()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_ContextualAnim_UpdateIKTargets);

	IKTargets.Reset();

	if (!Bindings.IsValid())
	{
		return;
	}

	const FContextualAnimSceneBinding* BindingPtr = Bindings.FindBindingByActor(GetOwner());
	if (BindingPtr == nullptr)
	{
		return;
	}

	const FAnimMontageInstance* MontageInstance = BindingPtr->GetAnimMontageInstance();
	if(MontageInstance == nullptr)
	{
		return;
	}

	const TArray<FContextualAnimIKTargetDefinition>& IKTargetDefs = Bindings.GetIKTargetDefContainerFromBinding(*BindingPtr).IKTargetDefs;

	for (const FContextualAnimIKTargetDefinition& IKTargetDef : IKTargetDefs)
	{
		float Alpha = 0.f;

		const EContextualAnimIKTargetAlphaProvider AlphaProvider = Bindings.GetSceneAsset()->GetIKTargetParams().AlphaProvider;
		if (AlphaProvider == EContextualAnimIKTargetAlphaProvider::AnimNotifyState)
		{
			Alpha = UAnimNotifyState_IKWindow::GetIKAlphaValue(IKTargetDef.GoalName, MontageInstance);
		}
		else if (AlphaProvider == EContextualAnimIKTargetAlphaProvider::Curve)
		{
			if (const UAnimInstance* AnimInstance = BindingPtr->GetAnimInstance())
			{
				Alpha = AnimInstance->GetCurveValue(IKTargetDef.GoalName);
			}
		}
		else if (AlphaProvider == EContextualAnimIKTargetAlphaProvider::None)
		{
			Alpha = 1.f;
		}
		else
		{
			check(false); // Unreachable
		}

		// @TODO: IKTargetTransform will be off by 1 frame if we tick before target. 
		// Should we at least add an option to the SceneAsset to setup tick dependencies or should this be entirely up to the user?

		if (const FContextualAnimSceneBinding* TargetBinding = Bindings.FindBindingByRole(IKTargetDef.TargetRoleName))
		{
			// Do not update if the target actor should be playing and animation but its not yet. 
			// This could happen in multi player when the initiator start playing the animation locally

			const UAnimSequenceBase* TargetAnimation = TargetBinding->GetAnimation();
			if (TargetAnimation)
			{
				//@TODO: Add support for dynamic montages
				const FAnimMontageInstance* TargetMontageInstance = TargetBinding->GetAnimMontageInstance();
				if (!TargetMontageInstance || TargetMontageInstance->Montage != TargetAnimation)
				{
					Alpha = 0.f;
				}
			}

			if (Alpha > 0.f)
			{
				if (const UMeshComponent* TargetMeshComp = UContextualAnimUtilities::TryGetMeshComponentWithSocket(TargetBinding->GetActor(), IKTargetDef.TargetBoneName))
				{
					if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Autogenerated)
					{
						const FTransform IKTargetParentTransform = TargetMeshComp->GetSocketTransform(IKTargetDef.TargetBoneName);

						const float Time = MontageInstance->GetPosition();
						const FTransform IKTargetTransform = Bindings.GetIKTargetTransformFromBinding(*BindingPtr, IKTargetDef.GoalName, Time) * IKTargetParentTransform;

						IKTargets.Add(FContextualAnimIKTarget(IKTargetDef.GoalName, IKTargetDef.BoneName, Alpha, IKTargetTransform));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						if (CVarContextualAnimIKDebug.GetValueOnGameThread() > 0)
						{
							const FColor Color = FColor::MakeRedToGreenColorFromScalar(Alpha);
							const float DrawDebugDuration = CVarContextualAnimIKDrawDebugLifetime.GetValueOnGameThread();
							DrawDebugLine(GetWorld(), IKTargetParentTransform.GetLocation(), IKTargetTransform.GetLocation(), Color, false, DrawDebugDuration, 0, 0.5f);
							DrawDebugCoordinateSystem(GetWorld(), IKTargetTransform.GetLocation(), IKTargetTransform.Rotator(), 10.f, false, DrawDebugDuration, 0, 0.5f);

							if (const USkeletalMeshComponent* SkelMesh = BindingPtr->GetSkeletalMeshComponent())
							{
								DrawDebugDirectionalArrow(GetWorld(), SkelMesh->GetSocketLocation(IKTargetDef.BoneName), IKTargetTransform.GetLocation(), 5.f, Color, false, DrawDebugDuration, 0, 0.5f);
							}
						}
#endif
					}
					else if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Bone)
					{
						const FTransform IKTargetTransform = TargetMeshComp->GetSocketTransform(IKTargetDef.TargetBoneName);

						IKTargets.Add(FContextualAnimIKTarget(IKTargetDef.GoalName, IKTargetDef.BoneName, Alpha, IKTargetTransform));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
						if (CVarContextualAnimIKDebug.GetValueOnGameThread() > 0)
						{
							const FColor Color = FColor::MakeRedToGreenColorFromScalar(Alpha);
							const float DrawDebugDuration = CVarContextualAnimIKDrawDebugLifetime.GetValueOnGameThread();
							DrawDebugCoordinateSystem(GetWorld(), IKTargetTransform.GetLocation(), IKTargetTransform.Rotator(), 10.f, false, DrawDebugDuration, 0, 0.5f);

							if (const USkeletalMeshComponent* SkelMesh = BindingPtr->GetSkeletalMeshComponent())
							{
								DrawDebugDirectionalArrow(GetWorld(), SkelMesh->GetSocketLocation(IKTargetDef.BoneName), IKTargetTransform.GetLocation(), 5.f, Color, false, DrawDebugDuration, 0, 0.5f);
							}
						}
#endif
					}
				}
			}
		}
	}
}

void UContextualAnimSceneActorComponent::AddIKGoals_Implementation(TMap<FName, FIKRigGoal>& OutGoals)
{
	OutGoals.Reserve(IKTargets.Num());

	for(const FContextualAnimIKTarget& IKTarget : IKTargets)
	{
		FIKRigGoal Goal;
		Goal.Name = IKTarget.GoalName;
		Goal.Position = IKTarget.Transform.GetLocation();
		Goal.Rotation = IKTarget.Transform.Rotator();
		Goal.PositionAlpha = IKTarget.Alpha;
		Goal.RotationAlpha = IKTarget.Alpha;
		Goal.PositionSpace = EIKRigGoalSpace::World;
		Goal.RotationSpace = EIKRigGoalSpace::World;
		OutGoals.Add(Goal.Name, Goal);
	}
}

const FContextualAnimIKTarget& UContextualAnimSceneActorComponent::GetIKTargetByGoalName(FName GoalName) const
{
	const FContextualAnimIKTarget* IKTargetPtr = IKTargets.FindByPredicate([GoalName](const FContextualAnimIKTarget& IKTarget){
		return IKTarget.GoalName == GoalName;
	});

	return IKTargetPtr ? *IKTargetPtr : FContextualAnimIKTarget::InvalidIKTarget;
}
