// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimSceneAsset.h"
#include "Animation/AnimMontage.h"
#include "BonePose.h"
#include "ContextualAnimUtilities.h"
#include "ContextualAnimSelectionCriterion.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimSceneAsset)

static FCompactPoseBoneIndex GetCompactPoseBoneIndexFromPose(const FCSPose<FCompactPose>& Pose, const FName& BoneName)
{
	const FBoneContainer& BoneContainer = Pose.GetPose().GetBoneContainer();
	for (int32 Idx = Pose.GetPose().GetNumBones() - 1; Idx >= 0; Idx--)
	{
		if (BoneContainer.GetReferenceSkeleton().GetBoneName(BoneContainer.GetBoneIndicesArray()[Idx]) == BoneName)
		{
			return FCompactPoseBoneIndex(Idx);
		}
	}

	checkf(false, TEXT("BoneName: %s Pose.Asset: %s Pose.NumBones: %d"), *BoneName.ToString(), *GetNameSafe(Pose.GetPose().GetBoneContainer().GetAsset()), Pose.GetPose().GetNumBones());
	return FCompactPoseBoneIndex(INDEX_NONE);
}

static void ExtractPoseIgnoringForceRootLock(UAnimSequenceBase* AnimSequenceBase, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose)
{
	UAnimSequence* AnimSequence = nullptr;
	if (UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimSequenceBase))
	{
		if (AnimMontage->SlotAnimTracks.Num() > 0)
		{
			const float ClampedTime = FMath::Clamp(Time, 0.f, AnimMontage->CalculateSequenceLength());
			if (FAnimSegment* Segment = AnimMontage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(ClampedTime))
			{
				AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference());
			}
		}
	}
	else
	{
		AnimSequence = Cast<UAnimSequence>(AnimSequenceBase);
	}

	if (AnimSequence)
	{
		TGuardValue<bool> ForceRootLockGuard(AnimSequence->bForceRootLock, false);
		UContextualAnimUtilities::ExtractComponentSpacePose(AnimSequenceBase, BoneContainer, Time, bExtractRootMotion, OutPose);
	}
}

// FContextualAnimSet
//==============================================================================================

int32 FContextualAnimSet::GetNumMandatoryRoles() const
{
	int32 Result = 0;
	for (const FContextualAnimTrack& AnimTrack : Tracks)
	{
		if (!AnimTrack.bOptional)
		{
			Result++;
		}
	}
	return Result;
}

// FContextualAnimSceneSection
//==============================================================================================

const FContextualAnimSet* FContextualAnimSceneSection::GetAnimSet(int32 AnimSetIdx) const
{
	return AnimSets.IsValidIndex(AnimSetIdx) ? &AnimSets[AnimSetIdx] : nullptr;
}

void FContextualAnimSceneSection::GenerateAlignmentTracks(UContextualAnimSceneAsset& SceneAsset)
{
	if(!SceneAsset.ShouldPrecomputeAlignmentTracks())
	{
		for (FContextualAnimSet& AnimSet : AnimSets)
		{
			for (FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
			{
				AnimTrack.AlignmentData.Empty();
			}
		}

		return;
	}

	// Necessary for FCompactPose that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
	// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits. 
	// UWorld::Tick adds a FMemMark to catch any allocation inside the game tick 
	// but any allocation from outside the game tick (like here when generating the alignment tracks off-line) must explicitly add a mark to avoid a leak 
	FMemMark Mark(FMemStack::Get());

	for (FContextualAnimSet& AnimSet : AnimSets)
	{
		// Generate alignment tracks relative to warp points
		for (FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
		{
			UE_LOGF(LogContextualAnim, Log, "Generating AlignmentTracks Tracks. Animation: %ls", *GetNameSafe(AnimTrack.Animation));

			const FTransform MeshToComponentInverse = FTransform(SceneAsset.GetMeshToComponentForRole(AnimTrack.Role).GetRotation()).Inverse();
			const float SampleInterval = 1.f / SceneAsset.GetSampleRate();

			// Initialize tracks for each alignment section
			const int32 TotalTracks = WarpPointDefinitions.Num();
			AnimTrack.AlignmentData.Initialize(TotalTracks, SampleInterval);
			for (int32 Idx = 0; Idx < TotalTracks; Idx++)
			{
				AnimTrack.AlignmentData.Tracks.TrackNames.Add(WarpPointDefinitions[Idx].WarpTargetName);
				AnimTrack.AlignmentData.Tracks.AnimationTracks.AddZeroed();
			}

			if (const UAnimSequenceBase* Animation = AnimTrack.Animation)
			{
				float Time = 0.f;
				float EndTime = Animation->GetPlayLength();
				int32 SampleIndex = 0;
				while (Time < EndTime)
				{
					Time = FMath::Clamp(SampleIndex * SampleInterval, 0.f, EndTime);
					SampleIndex++;

					const FTransform RootTransform = MeshToComponentInverse * (UContextualAnimUtilities::ExtractRootTransformFromAnimation(Animation, Time) * AnimTrack.MeshToScene);

					for (int32 Idx = 0; Idx < TotalTracks; Idx++)
					{
						FRawAnimSequenceTrack& AlignmentTrack = AnimTrack.AlignmentData.Tracks.AnimationTracks[Idx];

						const FTransform* WarpPointTransform = AnimSet.WarpPoints.Find(WarpPointDefinitions[Idx].WarpTargetName);
						const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

						AlignmentTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
						AlignmentTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
					}
				}
			}
			else
			{
				const FTransform RootTransform = MeshToComponentInverse * AnimTrack.MeshToScene;

				for (int32 Idx = 0; Idx < TotalTracks; Idx++)
				{
					FRawAnimSequenceTrack& SceneTrack = AnimTrack.AlignmentData.Tracks.AnimationTracks[Idx];

					const FTransform* WarpPointTransform = AnimSet.WarpPoints.Find(WarpPointDefinitions[Idx].WarpTargetName);
					const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

					SceneTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
					SceneTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
				}
			}
		}
	}
}

void FContextualAnimSceneSection::GenerateIKTargetTracks(UContextualAnimSceneAsset& SceneAsset)
{
	// @TODO: Optimize
	// We are generating IK Target tracks for the duration of the animation and for all the sets in the section
	// We should generate those tracks only for the animations with IK windows and only for the range of that window

	FMemMark Mark(FMemStack::Get());

	for (FContextualAnimSet& AnimSet : AnimSets)
	{
		for (FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
		{
			AnimTrack.IKTargetData.Empty();

			const TArray<FContextualAnimIKTargetDefinition>& IKTargetDefs = SceneAsset.GetIKTargetDefsForRole(AnimTrack.Role).IKTargetDefs;
			if (IKTargetDefs.Num() == 0)
			{
				continue;
			}

			if (UAnimSequenceBase* Animation = AnimTrack.Animation)
			{
				UE_LOGF(LogContextualAnim, Log, "Generating IK Target Tracks. Animation: %ls", *GetNameSafe(Animation));

				const float SampleInterval = 1.f / SceneAsset.GetSampleRate();

				TArray<FBoneIndexType> RequiredBoneIndexArray;

				// Helper structure to group pose extraction per target so we can extract the pose for all the bones that are relative to the same target in one pass
				struct FPoseExtractionHelper
				{
					const FContextualAnimTrack* TargetAnimTrackPtr = nullptr;
					TArray<TTuple<FName, FName, int32, FName, int32>> BonesData; //0: GoalName, 1: MyBoneName 2: MyBoneIndex, 3: TargetBoneName, 4: TargetBoneIndex
				};
				TMap<FName, FPoseExtractionHelper> PoseExtractionHelperMap;
				PoseExtractionHelperMap.Reserve(IKTargetDefs.Num());

				int32 TotalTracks = 0;
				for (const FContextualAnimIKTargetDefinition& IKTargetDef : IKTargetDefs)
				{
					if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Autogenerated)
					{
						const FName TargetRole = IKTargetDef.TargetRoleName;
						FPoseExtractionHelper* DataPtr = PoseExtractionHelperMap.Find(TargetRole);
						if (DataPtr == nullptr)
						{
							// Find track for target role. 
							const FContextualAnimTrack* TargetAnimTrackPtr = GetAnimTrack(AnimTrack.AnimSetIdx, TargetRole);
							if (TargetAnimTrackPtr == nullptr)
							{
								UE_LOGF(LogContextualAnim, Warning, "\t Can't find AnimTrack for TargetRole '%ls'", *TargetRole.ToString());
								continue;
							}

							DataPtr = &PoseExtractionHelperMap.Add(TargetRole);
							DataPtr->TargetAnimTrackPtr = TargetAnimTrackPtr;
						}

						const FName BoneName = IKTargetDef.BoneName;
						const int32 BoneIndex = Animation->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(BoneName);
						if (BoneIndex == INDEX_NONE)
						{
							UE_LOGF(LogContextualAnim, Warning, "\t Can't find BoneIndex. BoneName: %ls Animation: %ls Skel: %ls",
								*BoneName.ToString(), *GetNameSafe(Animation), *GetNameSafe(Animation->GetSkeleton()));

							continue;
						}

						// Find TargetBoneIndex. Note that we add TargetBoneIndex even if it is INDEX_NONE. In this case, my bone will be relative to the origin of the target actor. 
						// This is to support cases where the target actor doesn't have animation or TargetBoneName is None
						FName TargetBoneName = IKTargetDef.TargetBoneName;
						const UAnimSequenceBase* TargetAnimation = DataPtr->TargetAnimTrackPtr->Animation;
						const int32 TargetBoneIndex = TargetAnimation ? TargetAnimation->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(TargetBoneName) : INDEX_NONE;
						if (TargetBoneIndex == INDEX_NONE)
						{
							UE_LOGF(LogContextualAnim, Log, "\t Can't find TargetBoneIndex. BoneName: %ls Animation: %ls Skel: %ls. Track for this bone will be relative to the origin of the target role.",
								*TargetBoneName.ToString(), *GetNameSafe(TargetAnimation), TargetAnimation ? *GetNameSafe(TargetAnimation->GetSkeleton()) : nullptr);

							TargetBoneName = NAME_None;
						}

						RequiredBoneIndexArray.AddUnique(BoneIndex);

						DataPtr->BonesData.Add(MakeTuple(IKTargetDef.GoalName, BoneName, BoneIndex, TargetBoneName, TargetBoneIndex));
						TotalTracks++;

						UE_LOGF(LogContextualAnim, Log, "\t Bone added for extraction. GoalName: %ls BoneName: %ls (%d) TargetRole: %ls TargetAnimation: %ls TargetBone: %ls (%d)",
							*IKTargetDef.GoalName.ToString(), *BoneName.ToString(), BoneIndex, *TargetRole.ToString(), *GetNameSafe(TargetAnimation), *TargetBoneName.ToString(), TargetBoneIndex);
					}
				}

				if (TotalTracks > 0)
				{
					// Complete bones chain and create bone container to extract pose from my animation
					Animation->GetSkeleton()->GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);
					FBoneContainer BoneContainer = FBoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Animation->GetSkeleton());

					// Initialize track container
					AnimTrack.IKTargetData.Initialize(TotalTracks, SampleInterval);
					AnimTrack.IKTargetData.Tracks.TrackNames.AddZeroed(TotalTracks);
					AnimTrack.IKTargetData.Tracks.AnimationTracks.AddZeroed(TotalTracks);

					float Time = 0.f;
					float EndTime = Animation->GetPlayLength();
					int32 SampleIndex = 0;
					while (Time < EndTime)
					{
						Time = FMath::Clamp(SampleIndex * SampleInterval, 0.f, EndTime);
						SampleIndex++;

						// Extract pose from my animation
						FCSPose<FCompactPose> ComponentSpacePose;
						ExtractPoseIgnoringForceRootLock(Animation, BoneContainer, Time, false, ComponentSpacePose);

						// For each target role
						for (auto& Data : PoseExtractionHelperMap)
						{
							// Extract pose from target animation if any
							FCSPose<FCompactPose> OtherComponentSpacePose;
							TArray<FBoneIndexType> OtherRequiredBoneIndexArray;
							FBoneContainer OtherBoneContainer;
							UAnimSequenceBase* OtherAnimation = Data.Value.TargetAnimTrackPtr->Animation;
							if (OtherAnimation)
							{
								// Prepare array with the indices of the bones to extract from target animation
								OtherRequiredBoneIndexArray.Reserve(Data.Value.BonesData.Num());
								for (int32 Idx = 0; Idx < Data.Value.BonesData.Num(); Idx++)
								{
									const int32 TargetBoneIndex = Data.Value.BonesData[Idx].Get<4>();
									if (TargetBoneIndex != INDEX_NONE)
									{
										OtherRequiredBoneIndexArray.AddUnique(TargetBoneIndex);
									}
								}

								if (OtherRequiredBoneIndexArray.Num() > 0)
								{
									// Complete bones chain and create bone container to extract pose form the target animation
									OtherAnimation->GetSkeleton()->GetReferenceSkeleton().EnsureParentsExistAndSort(OtherRequiredBoneIndexArray);
									OtherBoneContainer = FBoneContainer(OtherRequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *OtherAnimation->GetSkeleton());

									// Extract pose from target animation
									ExtractPoseIgnoringForceRootLock(OtherAnimation, OtherBoneContainer, Time, false, OtherComponentSpacePose);
								}
							}

							for (int32 Idx = 0; Idx < Data.Value.BonesData.Num(); Idx++)
							{
								const FName TrackName = Data.Value.BonesData[Idx].Get<0>();
								AnimTrack.IKTargetData.Tracks.TrackNames[Idx] = TrackName;

								// Get bone transform from my animation
								const FName BoneName = Data.Value.BonesData[Idx].Get<1>();
								const FCompactPoseBoneIndex BoneIndex = GetCompactPoseBoneIndexFromPose(ComponentSpacePose, BoneName);
								const FTransform BoneTransform = (ComponentSpacePose.GetComponentSpaceTransform(BoneIndex) * AnimTrack.MeshToScene);

								// Get bone transform from target animation
								FTransform OtherBoneTransform = Data.Value.TargetAnimTrackPtr->MeshToScene;
								const FName TargetBoneName = Data.Value.BonesData[Idx].Get<3>();
								if (TargetBoneName != NAME_None)
								{
									const FCompactPoseBoneIndex OtherBoneIndex = GetCompactPoseBoneIndexFromPose(OtherComponentSpacePose, TargetBoneName);
									OtherBoneTransform = (OtherComponentSpacePose.GetComponentSpaceTransform(OtherBoneIndex) * Data.Value.TargetAnimTrackPtr->MeshToScene);
								}

								// Get transform relative to target
								const FTransform BoneRelativeToOther = BoneTransform.GetRelativeTransform(OtherBoneTransform);

								// Add transform to the track
								FRawAnimSequenceTrack& NewTrack = AnimTrack.IKTargetData.Tracks.AnimationTracks[Idx];
								NewTrack.PosKeys.Add(FVector3f(BoneRelativeToOther.GetLocation()));
								NewTrack.RotKeys.Add(FQuat4f(BoneRelativeToOther.GetRotation()));

								UE_LOGF(LogContextualAnim, Verbose, "\t\t Animation: %ls Time: %f BoneName: %ls (T: %ls) Target Animation: %ls TargetBoneName: %ls (T: %ls)",
									*GetNameSafe(Animation), Time, *BoneName.ToString(), *BoneTransform.GetLocation().ToString(),
									*GetNameSafe(OtherAnimation), *TargetBoneName.ToString(), *OtherBoneTransform.GetLocation().ToString());
							}
						}
					}
				}
			}
		}
	}
}

const FContextualAnimTrack* FContextualAnimSceneSection::GetAnimTrack(int32 AnimSetIdx, const FName& Role) const
{
	if (AnimSets.IsValidIndex(AnimSetIdx))
	{
		return AnimSets[AnimSetIdx].Tracks.FindByPredicate([Role](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role == Role; });
	}

	return nullptr;
}

const FContextualAnimTrack* FContextualAnimSceneSection::GetAnimTrack(int32 AnimSetIdx, int32 AnimTrackIdx) const
{
	if (AnimSets.IsValidIndex(AnimSetIdx))
	{
		const FContextualAnimSet& AnimSet = AnimSets[AnimSetIdx];
		if (AnimSet.Tracks.IsValidIndex(AnimTrackIdx))
		{
			return &AnimSet.Tracks[AnimTrackIdx];
		}
	}

	return nullptr;
}

FTransform FContextualAnimSceneSection::GetIKTargetTransformForRoleAtTime(int32 AnimSetIdx, FName Role, FName TrackName, float Time) const
{
	if (const FContextualAnimTrack* AnimTrack = GetAnimTrack(AnimSetIdx, Role))
	{
		return AnimTrack->IKTargetData.ExtractTransformAtTime(TrackName, Time);
	}

	return FTransform::Identity;
}

const FContextualAnimTrack* FContextualAnimSceneSection::FindFirstAnimTrackForRoleThatPassesSelectionCriteria(const FName& Role, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	for (const FContextualAnimSet& AnimSet : AnimSets)
	{
		for (const FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
		{
			if (AnimTrack.Role == Role)
			{
				bool bSuccess = true;
				for (const UContextualAnimSelectionCriterion* Criterion : AnimTrack.SelectionCriteria)
				{
					if (Criterion && !Criterion->DoesQuerierPassCondition(Primary, Querier))
					{
						bSuccess = false;
						break;
					}
				}

				if (bSuccess)
				{
					return &AnimTrack;
				}
			}
		}
	}

	return nullptr;
}

// UContextualAnimSceneAsset
//==============================================================================================

UContextualAnimSceneAsset::UContextualAnimSceneAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SampleRate = 15;
}

void UContextualAnimSceneAsset::PostLoad()
{
	Super::PostLoad();


	// Only do this during editor time because that is when we can fix it up.
// This means that properties will be fixed up during a cook!
#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

		if (CollisionBehavior_DEPRECATED != EContextualAnimCollisionBehavior::None)
		{
			UE_LOGF(LogContextualAnim, Log, "UContextualAnimSceneAsset::PostLoad - Collision Behavior Deprecation, Adding Entry For %ls (%ls)", *GetNameSafe(this), *UEnum::GetValueAsString(CollisionBehavior_DEPRECATED));

			FContextualAnimCollisionBehaviorParams BehaviourParams;
			BehaviourParams.CollisionBehavior = CollisionBehavior_DEPRECATED;
			BehaviourParams.CollisionChannelsToIgnoreParams = CollisionChannelsToIgnoreParams_DEPRECATED;

			CollisionBehaviors.Insert(BehaviourParams, 0);

			CollisionBehavior_DEPRECATED = EContextualAnimCollisionBehavior::None;
			CollisionChannelsToIgnoreParams_DEPRECATED.Empty();
		}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	
}

void UContextualAnimSceneAsset::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	PrecomputeData();
}

void UContextualAnimSceneAsset::PrecomputeData()
{
	if(!HasValidData())
	{
		return;
	}

	// Set indices on each AnimTrack
	for (int32 SectionIdx = 0; SectionIdx < Sections.Num(); SectionIdx++)
	{
		FContextualAnimSceneSection& Section = Sections[SectionIdx];
		for (int32 SetIdx = 0; SetIdx < Section.AnimSets.Num(); SetIdx++)
		{
			FContextualAnimSet& AnimSet = Section.AnimSets[SetIdx];
			for (int32 TrackIdx = 0; TrackIdx < AnimSet.Tracks.Num(); TrackIdx++)
			{
				FContextualAnimTrack& AnimTrack = AnimSet.Tracks[TrackIdx];
				AnimTrack.SectionIdx = SectionIdx;
				AnimTrack.AnimSetIdx = SetIdx;
				AnimTrack.AnimTrackIdx = TrackIdx;
			}
		}

		// Generate alignment tracks
		Section.GenerateAlignmentTracks(*this);

		// Generate IK Target tracks
		Section.GenerateIKTargetTracks(*this);
	}
}

const FContextualAnimTrack* UContextualAnimSceneAsset::FindAnimTrackByAnimation(const UAnimSequenceBase* Animation) const
{
	const FContextualAnimTrack* Result = nullptr;

	ForEachAnimTrack([&Result, Animation](const FContextualAnimTrack& AnimTrack)
		{
			if(AnimTrack.Animation == Animation)
			{
				Result = &AnimTrack;
				return UE::ContextualAnim::EForEachResult::Break;
			}

			return UE::ContextualAnim::EForEachResult::Continue;
		});

	return Result;
}

const FContextualAnimSceneSection* UContextualAnimSceneAsset::GetSection(int32 SectionIdx) const
{
	return (Sections.IsValidIndex(SectionIdx)) ? &Sections[SectionIdx] : nullptr;
}

const FContextualAnimSceneSection* UContextualAnimSceneAsset::GetSection(const FName& SectionName) const
{
	return Sections.FindByPredicate([&SectionName](const FContextualAnimSceneSection& Section) { return Section.Name == SectionName; });
}

const FContextualAnimSet* UContextualAnimSceneAsset::GetAnimSet(int32 SectionIdx, int32 AnimSetIdx) const
{
	const FContextualAnimSceneSection* Section = GetSection(SectionIdx);
	return Section ? Section->GetAnimSet(AnimSetIdx) : nullptr;
}

int32 UContextualAnimSceneAsset::GetSectionIndex(const FName& SectionName) const
{
	return Sections.IndexOfByPredicate([&SectionName](const FContextualAnimSceneSection& Section) { return Section.Name == SectionName; });
}

const FContextualAnimTrack* UContextualAnimSceneAsset::GetAnimTrack(int32 SectionIdx, int32 AnimSetIdx, const FName& Role) const
{
	return (Sections.IsValidIndex(SectionIdx)) ? Sections[SectionIdx].GetAnimTrack(AnimSetIdx, Role) : nullptr;
}

const FContextualAnimTrack* UContextualAnimSceneAsset::GetAnimTrack(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx) const
{
	return (Sections.IsValidIndex(SectionIdx)) ? Sections[SectionIdx].GetAnimTrack(AnimSetIdx, AnimTrackIdx) : nullptr;
}

void UContextualAnimSceneAsset::ForEachAnimTrack(FForEachAnimTrackFunction Function) const
{
	for (const FContextualAnimSceneSection& Section : Sections)
	{
		for (const FContextualAnimSet& Set : Section.AnimSets)
		{
			for (const FContextualAnimTrack& AnimTrack : Set.Tracks)
			{
				if (Function(AnimTrack) == UE::ContextualAnim::EForEachResult::Break)
				{
					return;
				}
			}
		}
	}
}

TArray<FName> UContextualAnimSceneAsset::GetRoles() const
{
 	TArray<FName> Result;

	if(RolesAsset)
	{
		for (const FContextualAnimRoleDefinition& RoleDef : RolesAsset->Roles)
		{
			Result.Add(RoleDef.Name);
		}
	}

 	return Result;
}

int32 UContextualAnimSceneAsset::GetNumMandatoryRoles(int32 SectionIdx, int32 AnimSetIdx) const
{
	const FContextualAnimSet* AnimSet = GetAnimSet(SectionIdx, AnimSetIdx);
	return AnimSet ? AnimSet->GetNumMandatoryRoles() : 0;
}

const FTransform& UContextualAnimSceneAsset::GetMeshToComponentForRole(const FName& Role) const
{
	if (RolesAsset)
	{
		if (const FContextualAnimRoleDefinition* RoleDef = RolesAsset->FindRoleDefinitionByName(Role))
		{
			return RoleDef->MeshToComponent;
		}
	}

	return FTransform::Identity;
}

TArray<FName> UContextualAnimSceneAsset::GetSectionNames() const
{
	TArray<FName> Result;

	for (const FContextualAnimSceneSection& Section : Sections)
	{
		Result.Add(Section.Name);
	}

	return Result;
}

bool UContextualAnimSceneAsset::Query(FName Role, FContextualAnimQueryResult& OutResult, const FContextualAnimQueryParams& QueryParams, const FTransform& ToWorldTransform) const
{
	//@TODO: Kept around only to do not break existing content. It will go away in the future.

	FContextualAnimSceneBindingContext Primary(ToWorldTransform);
	FContextualAnimSceneBindingContext Querier(QueryParams.Querier.IsValid() ? QueryParams.Querier->GetActorTransform() : QueryParams.QueryTransform);

	const FContextualAnimTrack* AnimTrack = Sections.IsValidIndex(0) ? Sections[0].FindFirstAnimTrackForRoleThatPassesSelectionCriteria(Role, Primary, Querier) : nullptr;
	if (AnimTrack)
	{
		OutResult.AnimSetIdx = AnimTrack->AnimSetIdx;
		OutResult.Animation = Cast<UAnimMontage>(AnimTrack->Animation);
		OutResult.EntryTransform = GetAlignmentTransform(*AnimTrack, 0, 0.f) * ToWorldTransform;
		OutResult.SyncTransform = GetAlignmentTransform(*AnimTrack, 0, AnimTrack->GetSyncTimeForWarpSection(0)) * ToWorldTransform;

		if (QueryParams.bFindAnimStartTime)
		{
			const FVector LocalLocation = (Querier.GetTransform().GetRelativeTransform(ToWorldTransform)).GetLocation();
			OutResult.AnimStartTime = FindBestAnimStartTime(*AnimTrack, LocalLocation);
		}

		return true;
	}

	return false;
}

float UContextualAnimSceneAsset::FindBestAnimStartTime(const FContextualAnimTrack& AnimTrack, const FVector& LocalLocation) const
{
	//@TODO: Kept around only to do not break existing content. It will go away in the future.

	float BestTime = 0.f;

	if (AnimTrack.AnimMaxStartTime < 0.f)
	{
		return BestTime;
	}

	const FVector SyncPointLocation = GetAlignmentTransform(AnimTrack, 0, AnimTrack.GetSyncTimeForWarpSection(0)).GetLocation();
	const float PerfectDistToSyncPointSq = GetAlignmentTransform(AnimTrack, 0, 0.f).GetTranslation().SizeSquared2D();
	const float ActualDistToSyncPointSq = FVector::DistSquared2D(LocalLocation, SyncPointLocation);

	if (ActualDistToSyncPointSq < PerfectDistToSyncPointSq)
	{
		float BestDistance = MAX_FLT;
		TArrayView<const FVector3f> PosKeys(AnimTrack.AlignmentData.Tracks.AnimationTracks[0].PosKeys.GetData(), AnimTrack.AlignmentData.Tracks.AnimationTracks[0].PosKeys.Num());

		//@TODO: Very simple search for now. Replace with Distance Matching + Pose Matching
		for (int32 Idx = 0; Idx < PosKeys.Num(); Idx++)
		{
			const float Time = Idx * AnimTrack.AlignmentData.SampleInterval;
			if (AnimTrack.AnimMaxStartTime > 0.f && Time >= AnimTrack.AnimMaxStartTime)
			{
				break;
			}

			const float DistFromCurrentFrameToSyncPointSq = FVector::DistSquared2D(SyncPointLocation, (FVector)PosKeys[Idx]);
			if (DistFromCurrentFrameToSyncPointSq < ActualDistToSyncPointSq)
			{
				BestTime = Time;
				break;
			}
		}
	}

	return BestTime;
}

int32 UContextualAnimSceneAsset::GetNumSections() const
{
	return Sections.Num();
}

int32 UContextualAnimSceneAsset::GetNumAnimSetsInSection(int32 SectionIdx) const
{
	return Sections.IsValidIndex(SectionIdx) ? Sections[SectionIdx].AnimSets.Num() : 0;
}

const FContextualAnimIKTargetDefContainer& UContextualAnimSceneAsset::GetIKTargetDefsForRole(const FName& Role) const
{
	const FContextualAnimIKTargetDefContainer* Ptr = IKTargetParams.IKTargetDefsForEachRole.FindByPredicate([Role](const FContextualAnimIKTargetDefContainer& Item) { return Item.Role == Role; });
	return Ptr ? *Ptr : FContextualAnimIKTargetDefContainer::EmptyContainer;
}

FTransform UContextualAnimSceneAsset::GetIKTargetTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, const FName& TrackName, float Time) const
{
	if (const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, AnimTrackIdx))
	{
		return AnimTrack->IKTargetData.ExtractTransformAtTime(TrackName, Time);
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetIKTargetTransform(const FContextualAnimSceneBinding& InBinding, const FName& TrackName, float Time) const
{
	return InBinding.GetIKTargetData().ExtractTransformAtTime(TrackName, Time);
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, int32 WarpPointIdx, float Time) const
{
	if (const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, AnimTrackIdx))
	{
		return GetAlignmentTransform(*AnimTrack, WarpPointIdx, Time);
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, const FName& WarpPointName, float Time) const
{
	if (const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, AnimTrackIdx))
	{
		return GetAlignmentTransform(*AnimTrack, WarpPointName, Time);
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransform(const FContextualAnimTrack& AnimTrack, int32 WarpPointIdx, float Time) const
{
	// If we are using precomputed alignment tracks, extract alignment transform from there. Otherwise, calculate them on the fly

	if (ShouldPrecomputeAlignmentTracks() && AnimTrack.AlignmentData.Tracks.GetNum() > 0)
	{
		return AnimTrack.AlignmentData.ExtractTransformAtTime(WarpPointIdx, Time);
	}

	const FContextualAnimSceneSection* Section = GetSection(AnimTrack.SectionIdx);
	if(Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimSceneAsset::GetAlignmentTransform Failed. Reason: Can't get AnimSceneSection. Asset: %ls Section Idx: %d", *GetNameSafe(this), AnimTrack.SectionIdx);
		return FTransform::Identity;
	}

	if (Section->GetWarpPointDefinitions().IsValidIndex(WarpPointIdx))
	{
		const FContextualAnimSet* AnimSet = GetAnimSet(AnimTrack.SectionIdx, AnimTrack.AnimSetIdx);
		if (AnimSet == nullptr)
		{
			UE_LOGF(LogContextualAnim, Warning, "UContextualAnimSceneAsset::GetAlignmentTransform Failed. Reason: Can't get AnimSet. Asset: %ls Section Idx: %d Anim Set Idx: %d", *GetNameSafe(this), AnimTrack.SectionIdx, AnimTrack.AnimSetIdx);
			return FTransform::Identity;
		}

		const FName WarpTargetName = Section->GetWarpPointDefinitions()[WarpPointIdx].WarpTargetName;
		if (const FTransform* WarpPointTransformPtr = AnimSet->WarpPoints.Find(WarpTargetName))
		{
			const FTransform MeshToComponentInverse = FTransform(GetMeshToComponentForRole(AnimTrack.Role).GetRotation()).Inverse();

			if (AnimTrack.Animation)
			{
				const FTransform RootTransform = MeshToComponentInverse * (UContextualAnimUtilities::ExtractRootTransformFromAnimation(AnimTrack.Animation, Time) * AnimTrack.MeshToScene);
				return RootTransform.GetRelativeTransform(*WarpPointTransformPtr);
			}
			else
			{
				const FTransform RootTransform = MeshToComponentInverse * AnimTrack.MeshToScene;
				return RootTransform.GetRelativeTransform(*WarpPointTransformPtr);
			}
		}
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransform(const FContextualAnimTrack& AnimTrack, const FName& WarpPointName, float Time) const
{
	const FContextualAnimSceneSection* Section = GetSection(AnimTrack.SectionIdx);
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimSceneAsset::GetAlignmentTransform Failed. Reason: Can't get AnimSceneSection. Asset: %ls Section Idx: %d", *GetNameSafe(this), AnimTrack.SectionIdx);
		return FTransform::Identity;
	}

	for (int32 Idx = 0; Idx < Section->GetWarpPointDefinitions().Num(); Idx++)
	{
		if (Section->GetWarpPointDefinitions()[Idx].WarpTargetName == WarpPointName)
		{
			return GetAlignmentTransform(AnimTrack, Idx, Time);
		}
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransformForRoleRelativeToOtherRole(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName OtherRole, float Time) const
{
	if(const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, Role))
	{
		if (const FContextualAnimTrack* OtherAnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, OtherRole))
		{
			const FTransform FromRoleRelativeToWarpPoint = GetAlignmentTransform(SectionIdx, AnimSetIdx, AnimTrack->AnimTrackIdx, 0, Time);
			const FTransform ToRoleRelativeToWarpPoint = GetAlignmentTransform(SectionIdx, AnimSetIdx, OtherAnimTrack->AnimTrackIdx, 0, Time);
			return FromRoleRelativeToWarpPoint.GetRelativeTransform(ToRoleRelativeToWarpPoint);
		}
	}
	
	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::GetAlignmentTransform(const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FName WarpPointName, float Time) const
{
	if (InBinding.GetAlignmentData().Tracks.GetNum() > 0)
	{
		return InBinding.GetAlignmentData().ExtractTransformAtTime(WarpPointName, Time);
	}

	const FContextualAnimSceneSection* Section = GetSection(InBinding.GetSectionIdx());
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimSceneAsset::GetAlignmentTransform Failed. Reason: Can't get AnimSceneSection. Asset: %ls Section Idx: %d", *GetNameSafe(this), InBinding.GetSectionIdx());
		return FTransform::Identity;
	}

	const FContextualAnimSet* AnimSet = GetAnimSet(InBinding.GetSectionIdx(), InBinding.GetAnimSetIdx());
	if (AnimSet == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimSceneAsset::GetAlignmentTransform Failed. Reason: Can't get AnimSet. Asset: %ls Section Idx: %d Anim Set Idx: %d", *GetNameSafe(this), InBinding.GetSectionIdx(), InBinding.GetAnimSetIdx());
		return FTransform::Identity;
	}

	const UAnimSequenceBase* Animation = InBinding.GetAnimation();

	if (const FTransform* WarpPointTransformPtr = AnimSet->WarpPoints.Find(WarpPointName))
	{
		// TODO: Add possible override the Role Mesh to Component As Well
		const FName Role = InBindings.GetRoleFromBinding(InBinding);
		const FTransform MeshToComponentInverse = FTransform(GetMeshToComponentForRole(Role).GetRotation()).Inverse();

		if (Animation)
		{
			const FTransform RootTransform = MeshToComponentInverse * (UContextualAnimUtilities::ExtractRootTransformFromAnimation(Animation, Time) * InBinding.GetMeshToScene());
			return RootTransform.GetRelativeTransform(*WarpPointTransformPtr);
		}
		else
		{
			const FTransform RootTransform = MeshToComponentInverse * InBinding.GetMeshToScene();
			return RootTransform.GetRelativeTransform(*WarpPointTransformPtr);
		}
	}
	return  FTransform::Identity;
}

const FContextualAnimTrack* UContextualAnimSceneAsset::FindAnimTrackForRoleWithClosestEntryLocation(int32 SectionIdx, const FName& Role, const FContextualAnimSceneBindingContext& Primary, const FVector& TestLocation) const
{
	const FContextualAnimTrack* Result = nullptr;

	float BestDistanceSq = MAX_FLT;

	if(const FContextualAnimSceneSection* Section = GetSection(SectionIdx))
	{
		for (const FContextualAnimSet& AnimSet : Section->AnimSets)
		{
			for (const FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
			{
				if (AnimTrack.Role == Role)
				{
					const FTransform EntryTransform = GetAlignmentTransform(AnimTrack, 0, 0.f) * Primary.GetTransform();
					const float DistSq = FVector::DistSquared(EntryTransform.GetLocation(), TestLocation);
					if (DistSq < BestDistanceSq)
					{
						BestDistanceSq = DistSq;
						Result = &AnimTrack;
					}
				}
			}
		}
	}

	return Result;
}

static FContextualAnimPoint GetContextualAnimPoint(const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimTrack& PrimaryAnimTrack, const FContextualAnimTrack& SecondaryAnimTrack, const FContextualAnimSceneBindingContext& PrimaryContext, int32 SampleRate, EContextualAnimPointType Type)
{
	check(SampleRate > 0);

	if (SecondaryAnimTrack.Animation)
	{
		const float Interval = 1.f / SampleRate;

		float T1 = 0.f;
		if(Type == EContextualAnimPointType::SyncFrame)
		{
			T1 = SecondaryAnimTrack.GetSyncTimeForWarpSection(0);
		}
		else if(Type == EContextualAnimPointType::LastFrame)
		{
			T1 = FMath::Max(SecondaryAnimTrack.Animation->GetPlayLength() - Interval, 0.f);
		}

		float T2 = T1 + Interval;

		const float Delta = (SecondaryAnimTrack.GetRootTransformAtTime(T2).GetTranslation() - SecondaryAnimTrack.GetRootTransformAtTime(T1).GetTranslation()).Size();
		const float Speed = Delta / Interval;

		const FTransform SecondaryRelativeToWarpPoint = SceneAsset.GetAlignmentTransform(SecondaryAnimTrack.SectionIdx, SecondaryAnimTrack.AnimSetIdx, SecondaryAnimTrack.AnimTrackIdx, 0, T1);
		const FTransform PrimaryRelativeToWarpPoint = SceneAsset.GetAlignmentTransform(PrimaryAnimTrack.SectionIdx, PrimaryAnimTrack.AnimSetIdx, PrimaryAnimTrack.AnimTrackIdx, 0, T1);
		const FTransform FinalTransform = SecondaryRelativeToWarpPoint.GetRelativeTransform(PrimaryRelativeToWarpPoint) * PrimaryContext.GetTransform();

		return FContextualAnimPoint(SecondaryAnimTrack.Role, FinalTransform, Speed, SecondaryAnimTrack.SectionIdx, SecondaryAnimTrack.AnimSetIdx, SecondaryAnimTrack.AnimTrackIdx);
	}
	else
	{
		return FContextualAnimPoint(SecondaryAnimTrack.Role, FTransform::Identity, 0.f, SecondaryAnimTrack.SectionIdx, SecondaryAnimTrack.AnimSetIdx, SecondaryAnimTrack.AnimTrackIdx);
	}
}

void UContextualAnimSceneAsset::GetAlignmentPointsForSecondaryRole(EContextualAnimPointType Type, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, TArray<FContextualAnimPoint>& OutResult) const
{
	if (Sections.IsValidIndex(SectionIdx))
	{
		OutResult.Reset(Sections[SectionIdx].AnimSets.Num());

		for (const FContextualAnimSet& AnimSet : Sections[SectionIdx].AnimSets)
		{			
			const FContextualAnimTrack* PrimaryAnimTrack = AnimSet.Tracks.FindByPredicate([this](const FContextualAnimTrack& AnimTrack){ return AnimTrack.Role == PrimaryRole; });
			const FContextualAnimTrack* SecondaryAnimTrack = AnimSet.Tracks.FindByPredicate([this](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role != PrimaryRole; });
			
			if(PrimaryAnimTrack && SecondaryAnimTrack)
			{
				OutResult.Add(GetContextualAnimPoint(*this, *PrimaryAnimTrack, *SecondaryAnimTrack, Primary, GetSampleRate(), Type));
			}
		}
	}
}

void UContextualAnimSceneAsset::GetAlignmentPointsForSecondaryRoleConsideringSelectionCriteria(EContextualAnimPointType Type, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier, EContextualAnimCriterionToConsider CriterionToConsider, TArray<FContextualAnimPoint>& OutResult) const
{
	if (Sections.IsValidIndex(SectionIdx))
	{
		OutResult.Reset(Sections[SectionIdx].AnimSets.Num());

		for (const FContextualAnimSet& AnimSet : Sections[SectionIdx].AnimSets)
		{
			const FContextualAnimTrack* PrimaryAnimTrack = AnimSet.Tracks.FindByPredicate([this](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role == PrimaryRole; });
			const FContextualAnimTrack* SecondaryAnimTrack = AnimSet.Tracks.FindByPredicate([this](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role != PrimaryRole; });

			if (PrimaryAnimTrack && SecondaryAnimTrack)
			{
				bool bSuccess = true;
				for (const UContextualAnimSelectionCriterion* Criterion : SecondaryAnimTrack->SelectionCriteria)
				{
					if (Criterion)
					{
						if ((CriterionToConsider == EContextualAnimCriterionToConsider::All) ||
							(CriterionToConsider == EContextualAnimCriterionToConsider::Spatial && Criterion->Type == EContextualAnimCriterionType::Spatial) ||
							(CriterionToConsider == EContextualAnimCriterionToConsider::Other && Criterion->Type == EContextualAnimCriterionType::Other))
						{
							if (!Criterion->DoesQuerierPassCondition(Primary, Querier))
							{
								bSuccess = false;
								break;
							}
						}
					}
				}

				if (bSuccess)
				{
					OutResult.Add(GetContextualAnimPoint(*this, *PrimaryAnimTrack, *SecondaryAnimTrack, Primary, GetSampleRate(), Type));
				}
			}
		}
	}
}

// Blueprint Interface
//------------------------------------------------------------------------------------------

UAnimSequenceBase* UContextualAnimSceneAsset::BP_FindAnimationForRole(int32 SectionIdx, int32 AnimSetIdx, FName Role) const
{
	const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, Role);
	return AnimTrack ? AnimTrack->Animation : nullptr;
}

FTransform UContextualAnimSceneAsset::BP_GetAlignmentTransformForRoleRelativeToWarpPoint(int32 SectionIdx, int32 AnimSetIdx, FName Role, float Time) const
{
	if (const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, Role))
	{
		return GetAlignmentTransform(AnimTrack->SectionIdx, AnimTrack->AnimSetIdx, AnimTrack->AnimTrackIdx, 0, Time);
	}

	return FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::BP_GetIKTargetTransformForRoleAtTime(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName TrackName, float Time) const
{
	return (Sections.IsValidIndex(SectionIdx)) ? Sections[SectionIdx].GetIKTargetTransformForRoleAtTime(AnimSetIdx, Role, TrackName, Time) : FTransform::Identity;
}

FTransform UContextualAnimSceneAsset::BP_GetIKTargetTransformForBindingAtTime(const FContextualAnimSceneBinding& InBinding, FName TrackName, float Time) const
{
	return InBinding.GetIKTargetData().ExtractTransformAtTime(TrackName, Time);
}

int32 UContextualAnimSceneAsset::BP_FindAnimSetIndexByAnimation(int32 SectionIdx, const UAnimSequenceBase* Animation) const
{
	if (Sections.IsValidIndex(SectionIdx))
	{
		for (const FContextualAnimSet& Set : Sections[SectionIdx].AnimSets)
		{
			for (const FContextualAnimTrack& AnimTrack : Set.Tracks)
			{
				if (AnimTrack.Animation == Animation)
				{
					return AnimTrack.AnimSetIdx;
				}
			}
		}
	}

	return INDEX_NONE;
}

void UContextualAnimSceneAsset::BP_GetStartAndEndTimeForWarpSection(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName WarpSectionName, float& OutStartTime, float& OutEndTime) const
{
	OutStartTime = OutEndTime = 0.f;
	if(const FContextualAnimTrack* AnimTrack = GetAnimTrack(SectionIdx, AnimSetIdx, Role))
	{
		AnimTrack->GetStartAndEndTimeForWarpSection(WarpSectionName, OutStartTime, OutEndTime);
	}
}

