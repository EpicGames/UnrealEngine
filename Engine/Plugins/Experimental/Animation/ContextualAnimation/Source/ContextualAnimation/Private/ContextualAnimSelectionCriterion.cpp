// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimSelectionCriterion.h"
#include "ContextualAnimSceneAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimSelectionCriterion)

UContextualAnimSelectionCriterion::UContextualAnimSelectionCriterion(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UContextualAnimSceneAsset* UContextualAnimSelectionCriterion::GetSceneAssetOwner() const
{
	return Cast<UContextualAnimSceneAsset>(GetOuter());
}

// UContextualAnimSelectionCriterion_Blueprint
//===========================================================================
UContextualAnimSelectionCriterion_Blueprint::UContextualAnimSelectionCriterion_Blueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer) 
{
}

const UContextualAnimSceneAsset* UContextualAnimSelectionCriterion_Blueprint::GetSceneAsset() const
{
	return GetSceneAssetOwner();
}

bool UContextualAnimSelectionCriterion_Blueprint::DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	return BP_DoesQuerierPassCondition(Primary, Querier);
}

// UContextualAnimSelectionCriterion_TriggerArea
//===========================================================================

UContextualAnimSelectionCriterion_TriggerArea::UContextualAnimSelectionCriterion_TriggerArea(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//@TODO: I think we could initialize this from the animations so it automatically creates an area from the primary to the owner of this criterion
	PolygonPoints.Add(FVector(100, -100, 0));
	PolygonPoints.Add(FVector(-100, -100, 0));
	PolygonPoints.Add(FVector(-100, 100, 0));
	PolygonPoints.Add(FVector(100, 100, 0));
}

bool UContextualAnimSelectionCriterion_TriggerArea::DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	check(PolygonPoints.Num() == 4);

	bool bResult = false;

	const FTransform PrimaryTransform = Primary.GetTransform();
	const FTransform QuerierTransform = Querier.GetTransform();

	const float HalfHeight = FMath::Max((Height / 2.f), 0.f);
	const float VDist = FMath::Abs((PrimaryTransform.GetLocation().Z + PolygonPoints[0].Z + HalfHeight) - QuerierTransform.GetLocation().Z);
	if (VDist <= HalfHeight)
	{
		const FVector2D TestPoint = FVector2D(QuerierTransform.GetLocation());
		const int32 NumPoints = PolygonPoints.Num();
		float AngleSum = 0.0f;
		for (int32 PointIndex = 0; PointIndex < NumPoints; ++PointIndex)
		{
			const FVector2D& VecAB = FVector2D(PrimaryTransform.TransformPositionNoScale(PolygonPoints[PointIndex])) - TestPoint;
			const FVector2D& VecAC = FVector2D(PrimaryTransform.TransformPositionNoScale(PolygonPoints[(PointIndex + 1) % NumPoints])) - TestPoint;
			const float Angle = FMath::Sign(FVector2D::CrossProduct(VecAB, VecAC)) * FMath::Acos(FMath::Clamp(FVector2D::DotProduct(VecAB, VecAC) / (VecAB.Size() * VecAC.Size()), -1.0f, 1.0f));
			AngleSum += Angle;
		}

		bResult = (FMath::Abs(AngleSum) > 0.001f);
	}

	return bResult;
}

// UContextualAnimSelectionCriterion_Cone
//===========================================================================

bool UContextualAnimSelectionCriterion_Cone::DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	bool bResult = false;

	const FTransform PrimaryTransform = Primary.GetTransform();
	const FTransform QuerierTransform = Querier.GetTransform();
	const float ActualDist = FVector::Dist(PrimaryTransform.GetLocation(), QuerierTransform.GetLocation());

	if(ActualDist <= Distance)
	{
		auto CheckCondition = [this](const FTransform& A, const FTransform& B)
		{
			const FVector VecA = (A.GetLocation() - B.GetLocation()).GetSafeNormal2D();
			const FVector VecB = B.GetRotation().GetForwardVector().RotateAngleAxis(Offset, FVector::UpVector);
			return FVector::DotProduct(VecA, VecB) > FMath::Cos(FMath::Clamp(FMath::DegreesToRadians(HalfAngle), 0.f, PI));
		};
		
		if (Mode == EContextualAnimCriterionConeMode::ToPrimary)
		{
			bResult = CheckCondition(PrimaryTransform, QuerierTransform);
		}
		else if (Mode == EContextualAnimCriterionConeMode::FromPrimary)
		{
			bResult = CheckCondition(QuerierTransform, PrimaryTransform);
		}
	}

	return bResult;
}

// UContextualAnimSelectionCriterion_Distance
//===========================================================================

bool UContextualAnimSelectionCriterion_Distance::DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	const FTransform PrimaryTransform = Primary.GetTransform();
	const FTransform QuerierTransform = Querier.GetTransform();

	float Distance = 0.f;
	if (Mode == EContextualAnimCriterionDistanceMode::Distance_2D)
	{
		Distance = FVector::Dist2D(PrimaryTransform.GetLocation(), QuerierTransform.GetLocation());
	}
	else if (Mode == EContextualAnimCriterionDistanceMode::Distance_3D)
	{
		Distance = FVector::Dist(PrimaryTransform.GetLocation(), QuerierTransform.GetLocation());
	}

	const bool bResult = FMath::IsWithinInclusive(Distance, MinDistance, MaxDistance);
	return bResult;
}
