// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimTypes.h"
#include "AnimationUtils.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimMontage.h"
#include "AnimNotifyState_MotionWarping.h"
#include "ContextualAnimUtilities.h"
#include "RootMotionModifier.h"
#include "ContextualAnimSelectionCriterion.h"
#include "ContextualAnimSceneActorComponent.h"
#include "ContextualAnimSceneAsset.h"
#include "Components/SkeletalMeshComponent.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "MotionWarpingComponent.h"
#include "Algo/RandomShuffle.h"
#include "ContextualAnimOverrideInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimTypes)

DEFINE_LOG_CATEGORY(LogContextualAnim);

const FContextualAnimSceneBinding FContextualAnimSceneBinding::InvalidBinding;

void FContextualAnimSceneBinding::SetOverrideData(const FContextualAnimationOverrideData& InData)
{
	Animation = InData.AnimSequence;
	MeshToScene = InData.MeshToScene;

	AlignmentData = InData.AlignmentData;
	IKTargetData = InData.IKTargetData;
}

const FContextualAnimTrack FContextualAnimTrack::EmptyTrack;
const FContextualAnimIKTarget FContextualAnimIKTarget::InvalidIKTarget;
const FContextualAnimIKTargetDefContainer FContextualAnimIKTargetDefContainer::EmptyContainer;
const FContextualAnimRoleDefinition FContextualAnimRoleDefinition::InvalidRoleDefinition;

// FContextualAnimAlignmentTrackContainer
///////////////////////////////////////////////////////////////////////

FTransform FContextualAnimAlignmentTrackContainer::ExtractTransformAtTime(const FName& TrackName, float Time) const
{
	const int32 TrackIndex = Tracks.TrackNames.IndexOfByKey(TrackName);
	return ExtractTransformAtTime(TrackIndex, Time);
}

FTransform FContextualAnimAlignmentTrackContainer::ExtractTransformAtTime(int32 TrackIndex, float Time) const
{
	FTransform AlignmentTransform = FTransform::Identity;

	if (Tracks.AnimationTracks.IsValidIndex(TrackIndex))
	{
		const FRawAnimSequenceTrack& Track = Tracks.AnimationTracks[TrackIndex];
		const int32 TotalFrames = Track.PosKeys.Num();
		const float TrackLength = (TotalFrames - 1) * SampleInterval;
		FAnimationUtils::ExtractTransformFromTrack(Track, Time, TotalFrames, TrackLength, EAnimInterpolationType::Linear, AlignmentTransform);
	}

	return AlignmentTransform;
}

float FContextualAnimTrack::GetSyncTimeForWarpSection(int32 WarpSectionIndex) const
{
	float StartTime, EndTime;
	GetStartAndEndTimeForWarpSection(WarpSectionIndex, StartTime, EndTime);
	return EndTime;
}

float FContextualAnimTrack::GetSyncTimeForWarpSection(const FName& WarpSectionName) const
{
	float StartTime, EndTime;
	GetStartAndEndTimeForWarpSection(WarpSectionName, StartTime, EndTime);
	return EndTime;
}

void FContextualAnimTrack::GetStartAndEndTimeForWarpSection(int32 WarpSectionIndex, float& OutStartTime, float& OutEndTime) const
{
	//@TODO: We need a better way to identify warping sections withing the animation. This is just a temp solution
	//@TODO: We should cache this data

	OutStartTime = 0.f;
	OutEndTime = 0.f;

	if (Animation && WarpSectionIndex >= 0)
	{
		FName LastWarpTargetName = NAME_None;
		int32 LastWarpSectionIndex = INDEX_NONE;

		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (const UAnimNotifyState_MotionWarping* Notify = Cast<const UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Modifier = Cast<const URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					const FName WarpTargetName = Modifier->WarpTargetName;
					if (WarpTargetName != NAME_None)
					{
						// First valid warping window. Initialize everything
						if (LastWarpSectionIndex == INDEX_NONE)
						{
							LastWarpTargetName = WarpTargetName;
							OutStartTime = NotifyEvent.GetTriggerTime();
							OutEndTime = NotifyEvent.GetEndTriggerTime();
							LastWarpSectionIndex = 0;
						}
						// If we hit another warping window but the sync point is the same as the previous, update SyncTime.
						// This is to deal with cases where a first short window is used to face the alignment point and a second one to perform the rest of the warp
						else if (WarpTargetName == LastWarpTargetName)
						{
							OutStartTime = NotifyEvent.GetTriggerTime();
							OutEndTime = NotifyEvent.GetEndTriggerTime();
						}
						// If we hit another warping window but with a different sync point name means that we have hit the first window of another warping section
						else
						{
							// If we haven't reached the desired WarpSection yet. Update control vars and keep moving
							if (WarpSectionIndex > LastWarpSectionIndex)
							{
								LastWarpTargetName = WarpTargetName;
								OutStartTime = NotifyEvent.GetTriggerTime();
								OutEndTime = NotifyEvent.GetEndTriggerTime();
								LastWarpSectionIndex++;
							}
							// Otherwise, stop here and return the value of the last window we found
							else
							{
								break;
							}
						}
					}
				}
			}
		}
	}
}

void FContextualAnimTrack::GetStartAndEndTimeForWarpSection(const FName& WarpSectionName, float& OutStartTime, float& OutEndTime) const
{
	//@TODO: We need a better way to identify warping sections within the animation. This is just a temp solution
	//@TODO: We should cache this data

	OutStartTime = 0.f;
	OutEndTime = 0.f;

	int32 Index = INDEX_NONE;
	float LastEndTime = 0.f;
	if (Animation && WarpSectionName != NAME_None)
	{
		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (const UAnimNotifyState_MotionWarping* Notify = Cast<const UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Config = Cast<const URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					const FName WarpTargetName = Config->WarpTargetName;
					if (WarpSectionName == WarpTargetName)
					{
						const float NotifyEndTriggerTime = NotifyEvent.GetEndTriggerTime();
						if(NotifyEndTriggerTime > LastEndTime)
						{
							LastEndTime = NotifyEndTriggerTime;
							Index = Idx;
						}
					}
				}
			}
		}
	}

	if(Index != INDEX_NONE)
	{
		const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Index];
		OutStartTime = NotifyEvent.GetTriggerTime();
		OutEndTime = NotifyEvent.GetEndTriggerTime();
	}
}

bool FContextualAnimTrack::DoesQuerierPassSelectionCriteria(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const
{
	for (const UContextualAnimSelectionCriterion* Criterion : SelectionCriteria)
	{
		if (Criterion && !Criterion->DoesQuerierPassCondition(Primary, Querier))
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t\t\t '%ls' condition check failed.", *GetNameSafe(Criterion));
			return false;
		}
		else
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t\t\t '%ls' condition check passed.", *GetNameSafe(Criterion));
		}
	}

	return true;
}

FTransform FContextualAnimTrack::GetRootTransformAtTime(float Time) const
{
	FTransform RootTransform = FTransform::Identity;
	if (Animation)
	{
		RootTransform = UContextualAnimUtilities::ExtractRootTransformFromAnimation(Animation, Time);
	}

	return RootTransform * MeshToScene;
}

// FContextualAnimSceneBindingContext
///////////////////////////////////////////////////////////////////////

void FContextualAnimSceneBindingContext::SetExternalTransform(const FTransform& InTransform)
{
	ExternalTransform = InTransform;
}

void FContextualAnimSceneBindingContext::AddGameplayTag(const FGameplayTag& Tag)
{
	ExternalGameplayTags.AddTag(Tag);
}

bool FContextualAnimSceneBindingContext::HasMatchingGameplayTag(const FGameplayTag& TagToCheck) const
{
	return ExternalGameplayTags.HasTag(TagToCheck);
}

bool FContextualAnimSceneBindingContext::HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	return ExternalGameplayTags.HasAll(TagContainer);
}

bool FContextualAnimSceneBindingContext::HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	return ExternalGameplayTags.HasAny(TagContainer);
}

FTransform FContextualAnimSceneBindingContext::GetTransform() const
{
	// If created with an external transform, used that one to represent the location/rotation of the actor
	if (ExternalTransform.IsSet())
	{
		return ExternalTransform.GetValue();
	}
	else if (UContextualAnimSceneActorComponent* Comp = GetSceneActorComponent())
	{
		if (AActor* OwnerActor = Comp->GetOwner())
		{
			return Comp->AlignmentOffset * OwnerActor->GetActorTransform();
		}
	}

	return FTransform::Identity;
}

FVector FContextualAnimSceneBindingContext::GetVelocity() const
{
	if (ExternalVelocity.IsSet())
	{
		return ExternalVelocity.GetValue();
	}
	else if (AActor* ActorPtr = GetActor())
	{
		return ActorPtr->GetVelocity();
	}

	return FVector::ZeroVector;
}

UContextualAnimSceneActorComponent* FContextualAnimSceneBindingContext::GetSceneActorComponent() const
{
	if (!CachedSceneActorComp.IsValid() || CachedSceneActorComp->GetOwner() != GetActor())
	{
		if (Actor.IsValid())
		{
			CachedSceneActorComp = Actor->FindComponentByClass<UContextualAnimSceneActorComponent>();
		}
	}

	return CachedSceneActorComp.Get();
}

UAnimInstance* FContextualAnimSceneBindingContext::GetAnimInstance() const
{
	if (!CachedAnimInstance.IsValid() || CachedAnimInstance->GetOwningActor() != GetActor())
	{
		CachedAnimInstance = UContextualAnimUtilities::TryGetAnimInstance(GetActor());
	}

	return CachedAnimInstance.Get();
}

USkeletalMeshComponent* FContextualAnimSceneBindingContext::GetSkeletalMeshComponent() const
{
	if (!CachedSkeletalMesh.IsValid() || CachedSkeletalMesh->GetOwner() != GetActor())
	{
		CachedSkeletalMesh = UContextualAnimUtilities::TryGetSkeletalMeshComponent(GetActor());
	}

	return CachedSkeletalMesh.Get();
}

UCharacterMovementComponent* FContextualAnimSceneBindingContext::GetCharacterMovementComponent() const
{
	if (!CachedMovementComp.IsValid() || CachedMovementComp->GetOwner() != GetActor())
	{
		if (Actor.IsValid())
		{
			CachedMovementComp = Actor->FindComponentByClass<UCharacterMovementComponent>();
		}
	}

	return CachedMovementComp.Get();
}

UMotionWarpingComponent* FContextualAnimSceneBindingContext::GetMotionWarpingComponent() const
{
	if (!CachedMotionWarpingComp.IsValid() || CachedMotionWarpingComp->GetOwner() != GetActor())
	{
		if (Actor.IsValid())
		{
			CachedMotionWarpingComp = Actor->FindComponentByClass<UMotionWarpingComponent>();
		}
	}

	return CachedMotionWarpingComp.Get();
}

// FContextualAnimSceneBinding
///////////////////////////////////////////////////////////////////////

FContextualAnimSceneBinding::FContextualAnimSceneBinding(const FContextualAnimSceneBindingContext& InContext, const FContextualAnimTrack& InAnimTrack)
{
	Context = InContext;
	SetAnimTrack(InAnimTrack);
}

FTransform FContextualAnimSceneBinding::GetIKTargetTransform(const FName& TrackName, float Time) const
{
	return IKTargetData.ExtractTransformAtTime(TrackName, Time);
}

FTransform FContextualAnimSceneBinding::GetAlignmentTransform(const FName& TrackName, float Time) const
{
	return AlignmentData.ExtractTransformAtTime(TrackName, Time);
}

void FContextualAnimSceneBinding::SetAnimTrack(const FContextualAnimTrack& InAnimTrack)
{
	SectionIdx = InAnimTrack.SectionIdx;
	AnimSetIdx = InAnimTrack.AnimSetIdx;
	AnimTrackIdx = InAnimTrack.AnimTrackIdx;
	Animation = InAnimTrack.Animation;
	AlignmentData = InAnimTrack.AlignmentData;
	IKTargetData = InAnimTrack.IKTargetData;
	MeshToScene = InAnimTrack.MeshToScene;
}

FAnimMontageInstance* FContextualAnimSceneBinding::GetAnimMontageInstance() const
{
	if (UAnimInstance* AnimInstance = GetAnimInstance())
	{
		return AnimInstance->GetActiveMontageInstance();
	}

	return nullptr;
}

float FContextualAnimSceneBinding::GetAnimMontageTime() const
{
	const FAnimMontageInstance* MontageInstance = GetAnimMontageInstance();
	return MontageInstance ? MontageInstance->GetPosition() : -1.f;
}

FName FContextualAnimSceneBinding::GetCurrentSection() const
{
	const FAnimMontageInstance* MontageInstance = GetAnimMontageInstance();
	return MontageInstance ? MontageInstance->GetCurrentSection() : NAME_None;
}

int32 FContextualAnimSceneBinding::GetCurrentSectionIndex() const
{
	if (const FAnimMontageInstance* MontageInstance = GetAnimMontageInstance())
	{
		float CurrentPosition;
		return MontageInstance->Montage->GetAnimCompositeSectionIndexFromPos(MontageInstance->GetPosition(), CurrentPosition);
	}

	return INDEX_NONE;
}

// FContextualAnimSceneBindings
///////////////////////////////////////////////////////////////////////

FContextualAnimSceneBindings::FContextualAnimSceneBindings(const UContextualAnimSceneAsset& InSceneAsset, int32 InSectionIdx, int32 InAnimSetIdx)
{
	if (ensureAlways(InSceneAsset.HasValidData()))
	{
		SceneAsset = &InSceneAsset;
		SectionIdx_DEPRECATED = InSectionIdx;
		AnimSetIdx_DEPRECATED = InAnimSetIdx;
		GenerateUniqueId();
	}
}

FContextualAnimSceneBindings::FContextualAnimSceneBindings(const UContextualAnimSceneAsset& InSceneAsset)
{
	if (ensureAlways(InSceneAsset.HasValidData()))
	{
		SceneAsset = &InSceneAsset;
		GenerateUniqueId();
	}
}

void FContextualAnimSceneBindings::GenerateUniqueId()
{
	static uint8 IncrementID = 0;
	if (IncrementID >= UINT8_MAX)
	{
		IncrementID = 0;
	}
	++IncrementID;
	Id = IncrementID;
}

void FContextualAnimSceneBindings::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (SceneAsset)
	{
		Collector.AddReferencedObject(SceneAsset);
	}
}

bool FContextualAnimSceneBindings::IsValid() const
{
	return Id != 0 && SceneAsset && SceneAsset->HasValidData() && Num() > 0;
}

void FContextualAnimSceneBindings::Reset()
{
	Id = 0;
	SceneAsset = nullptr;
	SectionIdx_DEPRECATED = INDEX_NONE;
	AnimSetIdx_DEPRECATED = INDEX_NONE;
	Data.Reset();
}

void FContextualAnimSceneBindings::Clear()
{
	Data.Reset();
}

bool FContextualAnimSceneBindings::ShouldSyncAnimation() const
{
	const FContextualAnimSceneBinding* SyncLeader = GetSyncLeader();
	const FContextualAnimSceneSection* Section = SyncLeader ? SceneAsset->GetSection(SyncLeader->GetSectionIdx()) : nullptr;
	if (Section == nullptr || Section->ShouldSyncAnimations() == false)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < Data.Num(); Idx++)
	{
		if (Data[Idx].GetActor() != SyncLeader->GetActor() && Data[Idx].GetSectionIdx() != SyncLeader->GetSectionIdx())
		{
			return false;
		}
	}

	return true;
}

bool FContextualAnimSceneBindings::ShouldBindingSyncAnimation(const FContextualAnimSceneBinding& Binding) const
{
	// Return true if the supplied Binding is not the SyncLeader, is in the same section as the SyncLeader 
	// and SyncAnimation is set to true in the Scene Asset for that section

	const FContextualAnimSceneBinding* SyncLeader = GetSyncLeader();
	const FContextualAnimSceneSection* Section = SyncLeader ? SceneAsset->GetSection(SyncLeader->GetSectionIdx()) : nullptr;
	return Section && Section->ShouldSyncAnimations() && SyncLeader->GetActor() != Binding.GetActor() && SyncLeader->GetSectionIdx() == Binding.GetSectionIdx();
}

bool FContextualAnimSceneBindings::AreAllBindingsInTheSameSection() const
{
	if (Data.Num() > 1)
	{
		const int8 SectionIdx = Data[0].GetSectionIdx();
		for (int32 Idx = 1; Idx < Data.Num(); Idx++)
		{
			if (Data[Idx].GetSectionIdx() != SectionIdx)
			{
				return false;
			}
		}
	}

	return true;
}

const FContextualAnimSceneBinding* FContextualAnimSceneBindings::GetSyncLeader() const
{
	//@TODO: Return first secondary binding as sync leader for now. This may have to be explicitly defined, either in the SceneAsset or when creating the bindings.
	return IsValid() ? Data.FindByPredicate([this](const FContextualAnimSceneBinding& Item) { return GetRoleFromBinding(Item) != SceneAsset->GetPrimaryRole(); }) : nullptr;
}

const FContextualAnimSceneBinding* FContextualAnimSceneBindings::GetPrimaryBinding() const
{
	return IsValid() ? Data.FindByPredicate([this](const FContextualAnimSceneBinding& Item) { return GetRoleFromBinding(Item) == SceneAsset->GetPrimaryRole(); }) : nullptr;
}

bool FContextualAnimSceneBindings::BindActorToRole(AActor& ActorRef, FName Role, int32 SectionIdx, int32 AnimSetIdx)
{
	const UContextualAnimSceneActorComponent* Comp = ActorRef.FindComponentByClass<UContextualAnimSceneActorComponent>();
	if (Comp == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::BindActorToRole. Failed to bind Actor: '%ls' to Role: '%ls'. Reason: Missing SceneActorComp",
			*GetNameSafe(&ActorRef), *Role.ToString());
		return false;
	}

	if (SceneAsset == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::BindActorToRole. Failed to bind Actor: '%ls' (Net Role: %d) to Role: '%ls'. Reason: Invalid SceneAsset",
			*GetNameSafe(&ActorRef), ActorRef.GetLocalRole(), *Role.ToString());
		return false;
	}

	if(const FContextualAnimSceneBinding* Binding = FindBindingByRole(Role))
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::BindActorToRole. Failed to bind Actor: '%ls' to Role: '%ls'. Reason: %ls already bound", 
			*GetNameSafe(&ActorRef), *Role.ToString(), *GetNameSafe(Binding->GetActor()));
		return false;
	}

	const FContextualAnimTrack* AnimTrackPtr = SceneAsset->GetAnimTrack(SectionIdx, AnimSetIdx, Role);
	if(AnimTrackPtr == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::BindActorToRole. Failed to bind Actor: '%ls' to Role: '%ls'. Reason: Can't find valid AnimTrack for it. SceneAsset: %ls SectionIdx: %d AnimSetIdx: %d",
			*GetNameSafe(&ActorRef), *Role.ToString(), *GetNameSafe(SceneAsset), SectionIdx, AnimSetIdx);
		return false;
	}

	Data.Add(FContextualAnimSceneBinding(FContextualAnimSceneBindingContext(&ActorRef), *AnimTrackPtr));

	UpdateBindingOverridesIfNeeded();

	return true;
}

const FContextualAnimTrack& FContextualAnimSceneBindings::GetAnimTrackFromBinding(const FContextualAnimSceneBinding& Binding) const
{
	if(!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::GetAnimTrackFromBinding Failed. Reason: Invalid Bindings. Bindings ID: %d Num: %d Scene Asset: %ls", 
			GetID(), Num(), *GetNameSafe(GetSceneAsset()));

		return FContextualAnimTrack::EmptyTrack;
	}

	if (const FContextualAnimTrack* AnimTrack = GetSceneAsset()->GetAnimTrack(Binding.SectionIdx, Binding.AnimSetIdx, Binding.AnimTrackIdx))
	{
		return *AnimTrack;
	}

	UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::GetAnimTrackFromBinding Failed. Reason: Can't find AnimTrack for binding. Bindings ID: %d Num: %d Scene Asset: %ls",
		GetID(), Num(), *GetNameSafe(GetSceneAsset()));
	return FContextualAnimTrack::EmptyTrack;
}

const FContextualAnimIKTargetDefContainer& FContextualAnimSceneBindings::GetIKTargetDefContainerFromBinding(const FContextualAnimSceneBinding& Binding) const
{
	if (!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::GetIKTargetDefContainerFromBinding Failed. Reason: Invalid Bindings. Bindings ID: %d Num: %d Scene Asset: %ls",
			GetID(), Num(), *GetNameSafe(GetSceneAsset()));

		return FContextualAnimIKTargetDefContainer::EmptyContainer;
	}

	return GetSceneAsset()->GetIKTargetDefsForRole(GetRoleFromBinding(Binding));
}

FTransform FContextualAnimSceneBindings::GetIKTargetTransformFromBinding(const FContextualAnimSceneBinding& Binding, const FName& TrackName, float Time) const
{
	if (!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::GetIKTargetTransformFromBinding Failed. Reason: Invalid Bindings. Bindings ID: %d Num: %d Scene Asset: %ls",
			GetID(), Num(), *GetNameSafe(GetSceneAsset()));

		return FTransform::Identity;
	}

	if (Binding.GetIKTargetData().Tracks.GetNum() > 0)
	{
		return Binding.GetIKTargetTransform(TrackName, Time);
	}

	return GetSceneAsset()->GetIKTargetTransform(Binding.SectionIdx, Binding.AnimSetIdx, Binding.AnimTrackIdx, TrackName, Time);
}

FTransform FContextualAnimSceneBindings::GetAlignmentTransformFromBinding(const FContextualAnimSceneBinding& Binding, const FName& TrackName, float Time) const
{
	if (!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::GetAlignmentTransformFromBinding Failed. Reason: Invalid Bindings. Bindings ID: %d Num: %d Scene Asset: %ls",
			GetID(), Num(), *GetNameSafe(GetSceneAsset()));

		return FTransform::Identity;
	}

	if (Binding.GetAlignmentData().Tracks.GetNum() > 0)
	{
		return Binding.GetAlignmentTransform(TrackName, Time);
	}

	return GetSceneAsset()->GetAlignmentTransform(Binding.SectionIdx, Binding.AnimSetIdx, Binding.AnimTrackIdx, TrackName, Time);
}

const FName& FContextualAnimSceneBindings::GetRoleFromBinding(const FContextualAnimSceneBinding& Binding) const
{
	return GetAnimTrackFromBinding(Binding).Role;
}

bool FContextualAnimSceneBindings::CheckConditions(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params)
{
	check(SceneAsset.HasValidData());

	auto DoCheck = [&SceneAsset, SectionIdx, AnimSetIdx](const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier, const FName& Role)
	{
		AActor* Actor = Querier.GetActor();
		UContextualAnimSceneActorComponent* SceneActorComp = Actor ? Actor->FindComponentByClass<UContextualAnimSceneActorComponent>() : nullptr;
		if (SceneActorComp == nullptr)
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t\t\t Missing ContextualAnimSceneActorComponent on the Actor.");
			return false;
		}
		
		const FContextualAnimTrack* AnimTrack = SceneAsset.GetAnimTrack(SectionIdx, AnimSetIdx, Role);
		if (AnimTrack == nullptr)
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t\t\t Missing AnimTrack in SceneAsset for role '%ls'.", *Role.ToString());
			return false;
		}

		if (AnimTrack->DoesQuerierPassSelectionCriteria(Primary, Querier) == false)
		{
			return false;
		}

		return true;
	};

	// Find the actor that should be bound to the primary Role.
	const FName PrimaryRole = SceneAsset.GetPrimaryRole();
	const FContextualAnimSceneBindingContext* PrimaryPtr = Params.Find(PrimaryRole);
	if (PrimaryPtr == nullptr)
	{
		UE_LOGF(LogContextualAnim, Verbose, "\t\t Missing actor to bind to the primary role '%ls'.", *PrimaryRole.ToString());
		return false;
	}

	// Test primary actor first
	// Passing the same data twice (as primary and querier) feels weird, but this allow us to run the selection mechanism even on the primary actor.
	UE_LOGF(LogContextualAnim, Verbose, "\t\t Checking binding conditions for primary role '%ls' (Actor: %ls)", *PrimaryRole.ToString(), *GetNameSafe(PrimaryPtr->GetActor()));

	if (DoCheck(*PrimaryPtr, *PrimaryPtr, PrimaryRole) == false)
	{
		UE_LOGF(LogContextualAnim, Verbose, "\t\t Actor '%ls' failed to pass some of the conditions for primary role '%ls'.", *GetNameSafe(PrimaryPtr->GetActor()), *PrimaryRole.ToString());
		return false;
	}
	else
	{
		UE_LOGF(LogContextualAnim, Verbose, "\t\t Actor '%ls' passed all the conditions for primary role '%ls'.", *GetNameSafe(PrimaryPtr->GetActor()), *PrimaryRole.ToString());
	}

	// Now test secondary actors
	int32 ValidEntries = 1;
	for (const auto& Pair : Params)
	{
		FName RoleToBind = Pair.Key;
		if (RoleToBind != PrimaryRole)
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t\t Checking binding conditions for role '%ls' (Actor: %ls)", *RoleToBind.ToString(), *GetNameSafe(Pair.Value.GetActor()));
			if (DoCheck(*PrimaryPtr, Pair.Value, RoleToBind) == false)
			{
				UE_LOGF(LogContextualAnim, Verbose, "\t\t Actor '%ls' failed to pass some of the conditions for role '%ls'.", *GetNameSafe(Pair.Value.GetActor()), *RoleToBind.ToString());
				return false;
			}
			else
			{
				UE_LOGF(LogContextualAnim, Verbose, "\t\t Actor '%ls' passed all the conditions for role '%ls'.", *GetNameSafe(Pair.Value.GetActor()), *RoleToBind.ToString());	
				ValidEntries++;
			}
		}
	}

	const int32 NumMandatoryRoles = SceneAsset.GetNumMandatoryRoles(SectionIdx, AnimSetIdx);
	if (ValidEntries < NumMandatoryRoles)
	{
		UE_LOGF(LogContextualAnim, Verbose, "\t Missing mandatory roles %d/%d. SceneAsset: %ls SectionIdx: %d AnimSetIdx: %d",
			ValidEntries, NumMandatoryRoles, *GetNameSafe(&SceneAsset), SectionIdx, AnimSetIdx);

		return false;
	}

	return true;
}

bool FContextualAnimSceneBindings::TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings)
{
	check(SceneAsset.HasValidData());
	
	OutBindings.Reset();

	if (FContextualAnimSceneBindings::CheckConditions(SceneAsset, SectionIdx, AnimSetIdx, Params))
	{
		OutBindings = FContextualAnimSceneBindings(SceneAsset);
		for (const auto& Pair : Params)
		{
			const FContextualAnimTrack* AnimTrackPtr = SceneAsset.GetAnimTrack(SectionIdx, AnimSetIdx, Pair.Key);
			check(AnimTrackPtr);

			OutBindings.Data.Add(FContextualAnimSceneBinding(Pair.Value, *AnimTrackPtr));
		}

		check(OutBindings.IsValid());
		return true;
	}

	return false;
}

bool FContextualAnimSceneBindings::TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings)
{
	check(SceneAsset.HasValidData());

	OutBindings.Reset();

	const TArray<FContextualAnimRoleDefinition>& Roles = SceneAsset.GetRolesAsset()->Roles;
	if (Roles.Num() > 2)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::TryCreateBindings Failed. Reason: Trying to create bindings with two actors for a SceneAsset with more than two roles. SceneAsset: %ls Num Roles: %d SectionIdx: %d AnimSetIdx: %d",
			*GetNameSafe(&SceneAsset), Roles.Num(), SectionIdx, AnimSetIdx);

		return false;
	}

	TMap<FName, FContextualAnimSceneBindingContext> Params;
	for(const FContextualAnimRoleDefinition& RoleDef : Roles)
	{
		const FName PrimaryRole = SceneAsset.GetPrimaryRole();
		Params.Add(RoleDef.Name, (RoleDef.Name == PrimaryRole) ? Primary : Secondary);
	}

	return TryCreateBindings(SceneAsset, SectionIdx, AnimSetIdx, Params, OutBindings);
}

int32 FContextualAnimSceneBindings::FindAnimSet(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params)
{
	check(SceneAsset.HasValidData());

	const FContextualAnimSceneSection* Section = SceneAsset.GetSection(SectionIdx);
	if (Section == nullptr)
	{
		return INDEX_NONE;
	}

	int32 AnimSetIdxSelected = INDEX_NONE;
	const int32 NumSets = Section->GetNumAnimSets();
	int32 NumValid = 0;
	
	UE_LOGF(LogContextualAnim, Verbose, "Find AnimSet in Section '%d' ('%ls'). NumSets: %d. SceneAsset: %ls", 
		SectionIdx, *Section->GetName().ToString(), NumSets, *GetNameSafe(&SceneAsset));

	if (NumSets == 1)
	{
		const FContextualAnimSet* AnimSet = Section->GetAnimSet(0);
		UE_LOGF(LogContextualAnim, Verbose, "\t Checking binding conditions for AnimSet '0' ('%ls').", *AnimSet->Name.ToString());

		if (CheckConditions(SceneAsset, SectionIdx, 0, Params))
		{
			AnimSetIdxSelected = 0;
			NumValid = 1;

			UE_LOGF(LogContextualAnim, Verbose, "\t AnimSet '0' ('%ls') can be selected", *AnimSet->Name.ToString());
		}
		else
		{
			UE_LOGF(LogContextualAnim, Verbose, "\t AnimSet '0' ('%ls') can NOT be selected.", *AnimSet->Name.ToString());
		}
	}
	else if (NumSets > 1)
	{
		TArray<TTuple<int32, float>, TInlineAllocator<5>> ValidSets; // 0: AnimSetIdx, 1: AnimSetRandomWeight

		float TotalWeight = 0;
		for (int32 AnimSetIdx = 0; AnimSetIdx < NumSets; AnimSetIdx++)
		{
			const FContextualAnimSet* AnimSet = Section->GetAnimSet(AnimSetIdx);
			UE_LOGF(LogContextualAnim, Verbose, "\t Checking binding conditions for AnimSet '%d' ('%ls').", AnimSetIdx, *AnimSet->Name.ToString());

			if (CheckConditions(SceneAsset, SectionIdx, AnimSetIdx, Params))
			{
				const float AnimSetRandomWeight = FMath::Max(AnimSet->RandomWeight, 0);
				ValidSets.Add(MakeTuple(AnimSetIdx, AnimSetRandomWeight));
				TotalWeight += AnimSetRandomWeight;

				UE_LOGF(LogContextualAnim, Verbose, "\t AnimSet '%d' ('%ls') can be selected. Adding it to the list of valid sets with a random weight of %.2f.", 
					AnimSetIdx, *AnimSet->Name.ToString(), AnimSetRandomWeight);
			}
			else
			{
				UE_LOGF(LogContextualAnim, Verbose, "\t AnimSet '%d' ('%ls') can NOT be selected.", 
					AnimSetIdx, *Section->GetAnimSet(AnimSetIdx)->Name.ToString());
			}
		}

		// Shuffle the list of valid sets to add an extra layer of randomness. 
		Algo::RandomShuffle(ValidSets);

		NumValid = ValidSets.Num();
		float RandomValue = FMath::RandRange(0.f, TotalWeight);
		for (int32 Idx = 0; Idx < NumValid; Idx++)
		{
			RandomValue -= FMath::Max(ValidSets[Idx].Get<1>(), 0);
			if (RandomValue <= 0)
			{
				AnimSetIdxSelected = ValidSets[Idx].Get<0>();
				break;
			}
		}
	}

	if (AnimSetIdxSelected != INDEX_NONE)
	{
		UE_LOGF(LogContextualAnim, Verbose, "Find AnimSet Result: SUCCESS. %d/%d AnimSets have passed the binding conditions. AnimSet '%d' ('%ls') was selected.", 
			NumValid, NumSets, AnimSetIdxSelected, *SceneAsset.GetSection(SectionIdx)->GetAnimSet(AnimSetIdxSelected)->Name.ToString());
	}
	else
	{
		UE_LOGF(LogContextualAnim, Verbose, "Find AnimSet Result: FAILURE. No AnimSet passed the binding conditions.");
	}

	return AnimSetIdxSelected;
}

bool FContextualAnimSceneBindings::TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings)
{
	check(SceneAsset.HasValidData());
	OutBindings.Reset();

	const int32 AnimSetIdxSelected = FindAnimSet(SceneAsset, SectionIdx, Params);
	if (AnimSetIdxSelected != INDEX_NONE)
	{
		OutBindings = FContextualAnimSceneBindings(SceneAsset);
		for (const auto& Pair : Params)
		{
			const FContextualAnimTrack* AnimTrackPtr = SceneAsset.GetAnimTrack(SectionIdx, AnimSetIdxSelected, Pair.Key);
			check(AnimTrackPtr);

			OutBindings.Data.Add(FContextualAnimSceneBinding(Pair.Value, *AnimTrackPtr));
		}

		check(OutBindings.IsValid());
		return true;
	}

	return false;
}

int32 FContextualAnimSceneBindings::FindAnimSetForTransitionTo(int32 NewSectionIdx) const
{
	if (!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::FindAnimSetForTransition Invalid Bindings");
		return INDEX_NONE;
	}

	const FContextualAnimSceneSection* Section = GetSceneAsset()->GetSection(NewSectionIdx);
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::FindAnimSetForTransition. Invalid NewSectionIdx %d", NewSectionIdx);
		return INDEX_NONE;
	}

	TMap<FName, FContextualAnimSceneBindingContext> Params;
	Params.Reserve(Num());
	for (const FContextualAnimSceneBinding& Binding : Data)
	{
		Params.Add(GetRoleFromBinding(Binding), Binding.GetContext());
	}

	const int32 AnimSetIdxSelected = FContextualAnimSceneBindings::FindAnimSet(*GetSceneAsset(), NewSectionIdx, Params);
	return AnimSetIdxSelected;
}

bool FContextualAnimSceneBindings::TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings)
{
	check(SceneAsset.HasValidData());

	OutBindings.Reset();

	const TArray<FContextualAnimRoleDefinition>& Roles = SceneAsset.GetRolesAsset()->Roles;
	if (Roles.Num() > 2)
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::TryCreateBindings Failed. Reason: Trying to create bindings with two actors for a SceneAsset with more than two roles. SceneAsset: %ls Num Roles: %d SectionIdx: %d",
			*GetNameSafe(&SceneAsset), Roles.Num(), SectionIdx);

		return false;
	}

	TMap<FName, FContextualAnimSceneBindingContext> Params;
	for (const FContextualAnimRoleDefinition& RoleDef : Roles)
	{
		const FName PrimaryRole = SceneAsset.GetPrimaryRole();
		Params.Add(RoleDef.Name, (RoleDef.Name == PrimaryRole) ? Primary : Secondary);
	}

	return TryCreateBindings(SceneAsset, SectionIdx, Params, OutBindings);
}


void FContextualAnimSceneBindings::CalculateWarpPoints(TArray<FContextualAnimWarpPoint>& OutWarpPoints) const
{
	// DEPRECATED: Use Section Index from Primary Binding until removed

	const FContextualAnimSceneBinding* Binding = IsValid() ? GetPrimaryBinding() : nullptr;
	if (Binding)
	{
		CalculateWarpPoints(OutWarpPoints, Binding->GetSectionIdx());
	}
}

void FContextualAnimSceneBindings::CalculateWarpPoints(TArray<FContextualAnimWarpPoint>& OutWarpPoints, int32 SectionIdx) const
{
	if (IsValid())
	{
		if (const FContextualAnimSceneSection* Section = GetSceneAsset()->GetSection(SectionIdx))
		{
			for (const FContextualAnimWarpPointDefinition& Def : Section->GetWarpPointDefinitions())
			{
				FContextualAnimWarpPoint& WarpPoint = OutWarpPoints.AddDefaulted_GetRef();
				CalculateWarpPoint(Def, WarpPoint);
			}
		}
	}
}

bool FContextualAnimSceneBindings::CalculateWarpPoint(const FContextualAnimWarpPointDefinition& WarpPointDef, FContextualAnimWarpPoint& OutWarpPoint) const
{
	if (!IsValid())
	{
		const UContextualAnimSceneAsset* Asset = SceneAsset.Get();
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Invalid Bindings. Id: %d Asset: %ls HasValidData: %d Num: %d", 
			GetID(), *GetNameSafe(Asset), Asset ? Asset->HasValidData() : 0, Num());  

		return false;
	}

	if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::PrimaryActor)
	{
		if(const FContextualAnimSceneBinding* PrimaryBinding = GetPrimaryBinding())
		{
			OutWarpPoint.Name = WarpPointDef.WarpTargetName;
			OutWarpPoint.Transform = PrimaryBinding->GetTransform();
			return true;
		}
		else
		{
			UE_LOGF(LogContextualAnim, VeryVerbose, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Can't find Primary Binding. Asset: %ls WarpPointDefinitionMode: PrimaryActor WarpTargetName: %ls",
				*GetNameSafe(SceneAsset.Get()), *WarpPointDef.WarpTargetName.ToString());
		}
	}
	else if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::Socket)
	{
		if (const FContextualAnimSceneBinding* PrimaryBinding = GetPrimaryBinding())
		{
			if (UMeshComponent* Component = UContextualAnimUtilities::TryGetMeshComponentWithSocket(PrimaryBinding->GetActor(), WarpPointDef.SocketName))
			{
				const FTransform SocketTransform = Component->GetSocketTransform(WarpPointDef.SocketName, ERelativeTransformSpace::RTS_Actor);
				FTransform WarpPointTransform = SocketTransform * PrimaryBinding->GetTransform();
				WarpPointTransform.SetScale3D(FVector(1.f));

				OutWarpPoint.Name = WarpPointDef.WarpTargetName;
				OutWarpPoint.Transform = WarpPointTransform;
				return true;
			}
			else
			{
				UE_LOGF(LogContextualAnim, VeryVerbose, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Can't find socket used as warp point in primary actor. Asset: %ls WarpPointDefinitionMode: Socket WarpTargetName: %ls SocketName: %ls Primary Actor: %ls",
					*GetNameSafe(SceneAsset.Get()), *WarpPointDef.WarpTargetName.ToString(), *WarpPointDef.SocketName.ToString(), *GetNameSafe(PrimaryBinding->GetActor()));
			}
		}
		else
		{
			UE_LOGF(LogContextualAnim, VeryVerbose, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Can't find Primary Binding. Asset: %ls WarpPointDefinitionMode: Socket WarpTargetName: %ls SocketName: %ls",
				*GetNameSafe(SceneAsset.Get()), *WarpPointDef.WarpTargetName.ToString(), *WarpPointDef.SocketName.ToString());
		}
	}
	else if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::Custom)
	{
		const FContextualAnimWarpPointCustomParams& Params = WarpPointDef.Params;

		if (const FContextualAnimSceneBinding* Binding = FindBindingByRole(Params.Origin))
		{
			OutWarpPoint.Name = WarpPointDef.WarpTargetName;
			if (Params.bAlongClosestDistance)
			{
				if (const FContextualAnimSceneBinding* OtherBinding = FindBindingByRole(Params.OtherRole))
				{
					const FTransform T1 = Binding->GetTransform();
					const FTransform T2 = OtherBinding->GetTransform();

					OutWarpPoint.Transform.SetLocation(FMath::Lerp<FVector>(T1.GetLocation(), T2.GetLocation(), Params.Weight));
					OutWarpPoint.Transform.SetRotation((T2.GetLocation() - T1.GetLocation()).GetSafeNormal2D().ToOrientationQuat());
					return true;
				}
				else
				{
					UE_LOGF(LogContextualAnim, VeryVerbose, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Can't find binding for Params.OtherRole. Asset: %ls WarpPointDefinitionMode: Custom WarpTargetName: %ls Params.OtherRole: %ls",
						*GetNameSafe(SceneAsset.Get()), *WarpPointDef.WarpTargetName.ToString(), *Params.OtherRole.ToString());
				}
			}
			else
			{
				OutWarpPoint.Transform = Binding->GetTransform();
				return true;
			}
		}
		else
		{
			UE_LOGF(LogContextualAnim, VeryVerbose, "FContextualAnimSceneBindings::CalculateWarpPoint failed. Reason: Can't find binding for Params.Origin. Asset: %ls WarpPointDefinitionMode: Custom WarpTargetName: %ls Params.Origin: %ls",
				*GetNameSafe(SceneAsset.Get()), *WarpPointDef.WarpTargetName.ToString(), *Params.Origin.ToString());
		}
	}

	return false;
}

void FContextualAnimSceneBindings::TransitionTo(int32 NewSectionIdx, int32 NewAnimSetIdx)
{
	if (!IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "FContextualAnimSceneBindings::TransitionTo NewSectionIdx: %d NewAnimSetIdx: %d Failed. Reason: Invalid Bindings. Bindings ID: %d Num: %d Scene Asset: %ls.",
			NewSectionIdx, NewAnimSetIdx, GetID(), Num(), *GetNameSafe(GetSceneAsset()));
		return;
	}

	if (const FContextualAnimSet* AnimSet = SceneAsset->GetAnimSet(NewSectionIdx, NewAnimSetIdx))
	{
		for (FContextualAnimSceneBinding& Binding : Data)
		{
			if (const FContextualAnimTrack* AnimTrack = SceneAsset->GetAnimTrack(NewSectionIdx, NewAnimSetIdx, GetRoleFromBinding(Binding)))
			{
				Binding.SetAnimTrack(*AnimTrack);
			}
		}

		UpdateBindingOverridesIfNeeded();
	}
}

bool FContextualAnimSceneBindings::RemoveActor(AActor& ActorRef)
{
	for (int32 Idx = 0; Idx < Data.Num(); Idx++)
	{
		if (Data[Idx].GetActor() == &ActorRef)
		{
			Data.RemoveAt(Idx);
			return true;
		}
	}

	return false;
}

void FContextualAnimSceneBindings::SetOverrideProvider(const TScriptInterface <IContextualAnimOverrideInterface>& Provider)
{
	OverrideProviderInterface = Provider;	
	UpdateBindingOverridesIfNeeded();
}

bool FContextualAnimSceneBindings::UpdateBindingOverridesIfNeeded()
{
	if (OverrideProviderInterface == nullptr)
	{
		return false;
	}

	if (SceneAsset == nullptr)
	{
		return false;
	}

	for (FContextualAnimSceneBinding& Binding : Data)
	{
		const FContextualAnimTrack* AnimTrack = SceneAsset->GetAnimTrack(Binding.SectionIdx, Binding.AnimSetIdx, GetRoleFromBinding(Binding));
		if (AnimTrack)
		{
			FContextualAnimationOverrideData OverrideData(*AnimTrack);
			if (OverrideProviderInterface->GetCASAnimationDataOverride(Binding, *this, *AnimTrack, OverrideData))
			{
				Binding.SetOverrideData(OverrideData);
			}
		}
	}

	for (FContextualAnimSceneBinding& Binding : Data)
	{
		// We need to "Recompute" these tracks with the updated data
		if (SceneAsset->ShouldPrecomputeAlignmentTracks())
		{
			UContextualAnimUtilities::GenerateAlignmentTrack(*SceneAsset, Binding, *this, Binding.GetAlignmentData());
		}

		UContextualAnimUtilities::GenerateIKTargetTrack(*SceneAsset, Binding, *this, Binding.GetIKTargetData());
	}

	return true;
}
