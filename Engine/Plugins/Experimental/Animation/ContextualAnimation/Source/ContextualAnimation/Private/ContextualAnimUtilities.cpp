// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimUtilities.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimInstance.h"
#include "BonePose.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"
#include "ContextualAnimSceneAsset.h"
#include "GameFramework/Character.h"
#include "ContextualAnimActorInterface.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "PrimitiveDrawingUtils.h"
#include "MotionWarpingComponent.h"
#include "AnimNotifyState_MotionWarping.h"
#include "RootMotionModifier.h"
#include "ContextualAnimSceneActorComponent.h"
#include "ContextualAnimOverrideInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimUtilities)

void UContextualAnimUtilities::ExtractLocalSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCompactPose& OutPose)
{
	OutPose.SetBoneContainer(&BoneContainer);

	FBlendedCurve Curve;
	Curve.InitFrom(BoneContainer);

	FAnimExtractContext Context(static_cast<double>(Time), bExtractRootMotion);

	UE::Anim::FStackAttributeContainer Attributes;
	FAnimationPoseData AnimationPoseData(OutPose, Curve, Attributes);
	if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		AnimSequence->GetBonePose(AnimationPoseData, Context);
	}
	else if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation))
	{
		const FAnimTrack& AnimTrack = AnimMontage->SlotAnimTracks[0].AnimTrack;
		AnimTrack.GetAnimationPose(AnimationPoseData, Context);
	}
}

void UContextualAnimUtilities::ExtractComponentSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose)
{
	FCompactPose Pose;
	ExtractLocalSpacePose(Animation, BoneContainer, Time, bExtractRootMotion, Pose);
	OutPose.InitPose(MoveTemp(Pose));
}

FTransform UContextualAnimUtilities::ExtractRootMotionFromAnimation(const UAnimSequenceBase* Animation, float StartTime, float EndTime)
{
	if (const UAnimMontage* Anim = Cast<UAnimMontage>(Animation))
	{
		return Anim->ExtractRootMotionFromTrackRange(StartTime, EndTime, FAnimExtractContext());
	}

	if (const UAnimSequence* Anim = Cast<UAnimSequence>(Animation))
	{
		return Anim->ExtractRootMotionFromRange(StartTime, EndTime, FAnimExtractContext());
	}

	return FTransform::Identity;
}

FTransform UContextualAnimUtilities::ExtractRootTransformFromAnimation(const UAnimSequenceBase* Animation, float Time)
{
	if (const UAnimMontage* AnimMontage = Cast<UAnimMontage>(Animation))
	{
		if (const FAnimSegment* Segment = AnimMontage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(Time))
		{
			if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference()))
			{
				const float AnimSequenceTime = Segment->ConvertTrackPosToAnimPos(Time);
				return AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(AnimSequenceTime)), nullptr);
			}
		}
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		return AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(static_cast<double>(Time)), nullptr);
	}

	return FTransform::Identity;
}

void UContextualAnimUtilities::BP_DrawDebugPose(const UObject* WorldContextObject, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness)
{
	if(GEngine)
	{
		if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
		{
			DrawPose(World, Animation, Time, LocalToWorldTransform, Color, LifeTime, Thickness);
		}
	}
}

void UContextualAnimUtilities::DrawPose(const UWorld* World, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness)
{
	if (World)
	{
		auto DrawFunction = [World](const FVector& LineStart, const FVector& LineEnd, const FColor& Color, float LifeTime, float Thickness) {
			DrawDebugLine(World, LineStart, LineEnd, Color, false, LifeTime, 0, Thickness);
		};

		DrawPose(Animation, Time, LocalToWorldTransform, Color, LifeTime, Thickness, DrawFunction);
	}
}

void UContextualAnimUtilities::DrawPose(FPrimitiveDrawInterface* PDI, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float Thickness)
{
	if (PDI)
	{
		auto DrawFunction = [PDI](const FVector& LineStart, const FVector& LineEnd, const FColor& Color, float LifeTime, float Thickness) {
			PDI->DrawLine(LineStart, LineEnd, Color, 0, Thickness);
		};

		DrawPose(Animation, Time, LocalToWorldTransform, Color, 0, Thickness, DrawFunction);
	}
}

void UContextualAnimUtilities::DrawPose(const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness, FDrawLineFunction DrawFunction)
{
	FMemMark Mark(FMemStack::Get());

	Time = FMath::Clamp(Time, 0.f, Animation->GetPlayLength());

	const int32 TotalBones = Animation->GetSkeleton()->GetReferenceSkeleton().GetNum();
	TArray<FBoneIndexType> RequiredBoneIndexArray;
	RequiredBoneIndexArray.Reserve(TotalBones);
	for (int32 Idx = 0; Idx < TotalBones; Idx++)
	{
		RequiredBoneIndexArray.Add(Idx);
	}

	FBoneContainer BoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Animation->GetSkeleton());
	FCSPose<FCompactPose> ComponentSpacePose;
	UContextualAnimUtilities::ExtractComponentSpacePose(Animation, BoneContainer, Time, true, ComponentSpacePose);

	for (int32 Index = 0; Index < ComponentSpacePose.GetPose().GetNumBones(); ++Index)
	{
		const FCompactPoseBoneIndex CompactPoseBoneIndex = FCompactPoseBoneIndex(Index);
		const FCompactPoseBoneIndex ParentIndex = ComponentSpacePose.GetPose().GetParentBoneIndex(CompactPoseBoneIndex);
		FVector Start, End;

		const FTransform Transform = ComponentSpacePose.GetComponentSpaceTransform(CompactPoseBoneIndex) * LocalToWorldTransform;

		if (ParentIndex.GetInt() >= 0)
		{
			Start = (ComponentSpacePose.GetComponentSpaceTransform(ParentIndex) * LocalToWorldTransform).GetLocation();
			End = Transform.GetLocation();
		}
		else
		{
			Start = LocalToWorldTransform.GetLocation();
			End = Transform.GetLocation();
		}

		DrawFunction(Start, End, Color.ToFColor(false), LifeTime, Thickness);
	}
}

void UContextualAnimUtilities::DrawDebugAnimSet(const UWorld* World, const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSet& AnimSet, float Time, const FTransform& ToWorldTransform, const FColor& Color, float LifeTime, float Thickness)
{
	if (World)
	{
		for (const FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
		{
			const FTransform Transform = (FTransform(SceneAsset.GetMeshToComponentForRole(AnimTrack.Role).GetRotation()) * SceneAsset.GetAlignmentTransform(AnimTrack, 0, Time)) * ToWorldTransform;

			if (const UAnimSequenceBase* Animation = AnimTrack.Animation)
			{
				DrawPose(World, Animation, Time, Transform, Color, LifeTime, Thickness);
			}
			else
			{
				DrawDebugCoordinateSystem(World, Transform.GetLocation(), Transform.Rotator(), 50.f, false, LifeTime, 0, Thickness);
			}
		}
	}
}

const FAnimNotifyEvent* UContextualAnimUtilities::FindFirstWarpingWindowForWarpTarget(const UAnimSequenceBase* Animation, FName WarpTargetName)
{
	if(Animation)
	{
		return Animation->Notifies.FindByPredicate([WarpTargetName](const FAnimNotifyEvent& NotifyEvent)
		{
			if (const UAnimNotifyState_MotionWarping* Notify = Cast<UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Modifier = Cast<URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					if (Modifier->WarpTargetName == WarpTargetName)
					{
						return true;
					}
				}
			}

			return false;
		});
	}

	return nullptr;
}

UMeshComponent* UContextualAnimUtilities::TryGetMeshComponentWithSocket(const AActor* Actor, FName SocketName)
{
	if (Actor)
	{
		TInlineComponentArray<UMeshComponent*> Components(Actor);
		for (UMeshComponent* Component : Components)
		{
			if (Component && Component->DoesSocketExist(SocketName))
			{
				return Component;
			}
		}
	}

	return nullptr;
}

USkeletalMeshComponent* UContextualAnimUtilities::TryGetSkeletalMeshComponent(const AActor* Actor)
{
	USkeletalMeshComponent* SkelMeshComp = nullptr;
	if (Actor)
	{
		if (const ACharacter* Character = Cast<const ACharacter>(Actor))
		{
			SkelMeshComp = Character->GetMesh();
		}
		else if (Actor->GetClass()->ImplementsInterface(UContextualAnimActorInterface::StaticClass()))
		{
			SkelMeshComp = IContextualAnimActorInterface::Execute_GetMesh(Actor);
		}
		else
		{
			SkelMeshComp = Actor->FindComponentByClass<USkeletalMeshComponent>();
		}
	}

	return SkelMeshComp;
}

UAnimInstance* UContextualAnimUtilities::TryGetAnimInstance(const AActor* Actor)
{
	if (USkeletalMeshComponent* SkelMeshComp = UContextualAnimUtilities::TryGetSkeletalMeshComponent(Actor))
	{
		return SkelMeshComp->GetAnimInstance();
	}

	return nullptr;
}

FAnimMontageInstance* UContextualAnimUtilities::TryGetActiveAnimMontageInstance(const AActor* Actor)
{
	if(UAnimInstance* AnimInstance = UContextualAnimUtilities::TryGetAnimInstance(Actor))
	{
		return AnimInstance->GetActiveMontageInstance();
	}

	return nullptr;
}

float UContextualAnimUtilities::GetSyncTimeForWarpSection(const UAnimSequenceBase* Animation, const FName& WarpSectionName)
{
	float StartTime, EndTime;
	GetStartAndEndTimeForWarpSection(Animation, WarpSectionName, StartTime, EndTime);
	return EndTime;
}

void UContextualAnimUtilities::GetStartAndEndTimeForWarpSection(const UAnimSequenceBase* Animation, const FName& WarpSectionName, float& OutStartTime, float& OutEndTime)
{
	//@TODO: We need a better way to identify warping sections within the animation. This is just a temp solution
	//@TODO: We should cache this data

	OutStartTime = 0.f;
	OutEndTime = 0.f;

	int32 Index = INDEX_NONE;
	float LastEndTime = 0.f;
	if (Animation && WarpSectionName != NAME_None)
	{
		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (const UAnimNotifyState_MotionWarping* Notify = Cast<const UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Config = Cast<const URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					const FName WarpTargetName = Config->WarpTargetName;
					if (WarpSectionName == WarpTargetName)
					{
						const float NotifyEndTriggerTime = NotifyEvent.GetEndTriggerTime();
						if (NotifyEndTriggerTime > LastEndTime)
						{
							LastEndTime = NotifyEndTriggerTime;
							Index = Idx;
						}
					}
				}
			}
		}
	}

	if (Index != INDEX_NONE)
	{
		const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Index];
		OutStartTime = NotifyEvent.GetTriggerTime();
		OutEndTime = NotifyEvent.GetEndTriggerTime();
	}
}

FName UContextualAnimUtilities::GetStartAndEndTimeForWarpSection(const UAnimSequenceBase* Animation, int32 DesiredWarpSectionOrdinal, float& OutStartTime, float& OutEndTime)
{
	OutStartTime = 0.f;
	OutEndTime = 0.f;

	if (Animation && DesiredWarpSectionOrdinal >= 0)
	{
		FName CurrentSectionName = NAME_None;
		int32 CurrentSectionOrdinal = INDEX_NONE;

		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (const UAnimNotifyState_MotionWarping* Notify = Cast<const UAnimNotifyState_MotionWarping>(NotifyEvent.NotifyStateClass))
			{
				if (const URootMotionModifier_Warp* Modifier = Cast<const URootMotionModifier_Warp>(Notify->RootMotionModifier))
				{
					const FName WarpTargetName = Modifier->WarpTargetName;
					if (WarpTargetName != NAME_None)
					{
						// First valid warping window. Initialize everything
						if (CurrentSectionOrdinal == INDEX_NONE)
						{
							CurrentSectionName = WarpTargetName;
							CurrentSectionOrdinal = 0;
							OutStartTime = NotifyEvent.GetTriggerTime();
							OutEndTime = NotifyEvent.GetEndTriggerTime();
						}
						// Same section (e.g. a rotation window followed by the main warp window).
						// Keep the earliest start time, extend end time to cover all sub-windows.
						else if (WarpTargetName == CurrentSectionName)
						{
							OutEndTime = FMath::Max(OutEndTime, NotifyEvent.GetEndTriggerTime());
						}
						// Different warp target name means we've hit the first window of a new warp section
						else
						{
							// We already found the desired section and this is a different one, so we're done
							// Note this assumes notifies are grouped. E.g, we don't have A B A
							if (CurrentSectionOrdinal == DesiredWarpSectionOrdinal)
							{
								break;
							}

							// Haven't reached the desired section yet. Advance to this new section and keep going.
							CurrentSectionName = WarpTargetName;
							CurrentSectionOrdinal++;
							OutStartTime = NotifyEvent.GetTriggerTime();
							OutEndTime = NotifyEvent.GetEndTriggerTime();
						}
					}
				}
			}
		}

		// Requested ordinal was beyond the number of sections found
		if (CurrentSectionOrdinal != DesiredWarpSectionOrdinal)
		{
			OutStartTime = 0.f;
			OutEndTime = 0.f;
			return NAME_None;
		}

		return CurrentSectionName;
	}

	return NAME_None;
}

void UContextualAnimUtilities::GenerateAlignmentTrack(const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FContextualAnimAlignmentTrackContainer& Container)
{
	if (!SceneAsset.ShouldPrecomputeAlignmentTracks())
	{
		Container.Empty();
		return;
	}

	// Necessary for FCompactPose that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
	// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits. 
	// UWorld::Tick adds a FMemMark to catch any allocation inside the game tick 
	// but any allocation from outside the game tick (like here when generating the alignment tracks off-line) must explicitly add a mark to avoid a leak 
	FMemMark Mark(FMemStack::Get());

	//UE_LOGF(LogContextualAnim, Log, "Generating AlignmentTracks Tracks. Animation: %ls", *GetNameSafe(AnimTrack.Animation));

	int32 SectionIndex = InBinding.GetSectionIdx();
	int32 AnimSetIndex = InBinding.GetAnimSetIdx();

	const FContextualAnimSceneSection* AnimSection = SceneAsset.GetSection(SectionIndex);
	const FContextualAnimSet* AnimSet = AnimSection->GetAnimSet(AnimSetIndex);

	const FName& Role = InBindings.GetRoleFromBinding(InBinding);

	const FTransform MeshToComponentInverse = FTransform(SceneAsset.GetMeshToComponentForRole(Role).GetRotation()).Inverse();
	const float SampleInterval = 1.f / SceneAsset.GetSampleRate();

	// Initialize tracks for each alignment section
	const int32 TotalTracks = AnimSection->GetWarpPointDefinitions().Num();
	Container.Initialize(TotalTracks, SampleInterval);
	for (int32 Idx = 0; Idx < TotalTracks; Idx++)
	{
		Container.Tracks.TrackNames.Add(AnimSection->GetWarpPointDefinitions()[Idx].WarpTargetName);
		Container.Tracks.AnimationTracks.AddZeroed();
	}

	if (const UAnimSequenceBase* Animation = InBinding.GetAnimation())
	{
		float Time = 0.f;
		float EndTime = Animation->GetPlayLength();
		int32 SampleIndex = 0;
		while (Time < EndTime)
		{
			Time = FMath::Clamp(SampleIndex * SampleInterval, 0.f, EndTime);
			SampleIndex++;

			const FTransform RootTransform = MeshToComponentInverse * (UContextualAnimUtilities::ExtractRootTransformFromAnimation(Animation, Time) * InBinding.GetMeshToScene());

			for (int32 Idx = 0; Idx < TotalTracks; Idx++)
			{
				const FTransform* WarpPointTransform = AnimSet->WarpPoints.Find(AnimSection->GetWarpPointDefinitions()[Idx].WarpTargetName);
				const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

				FRawAnimSequenceTrack& AlignmentTrack = Container.Tracks.AnimationTracks[Idx];
				AlignmentTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
				AlignmentTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
			}
		}
	}
	else
	{
		const FTransform RootTransform = MeshToComponentInverse *InBinding.GetMeshToScene();

		for (int32 Idx = 0; Idx < TotalTracks; Idx++)
		{
			const FTransform* WarpPointTransform = AnimSet->WarpPoints.Find(AnimSection->GetWarpPointDefinitions()[Idx].WarpTargetName);
			const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

			FRawAnimSequenceTrack& SceneTrack = Container.Tracks.AnimationTracks[Idx];
			SceneTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
			SceneTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
		}
	}
}

void UContextualAnimUtilities::ExtractPoseIgnoringForceRootLock(UAnimSequenceBase* AnimSequenceBase, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose)
{
	UAnimSequence* AnimSequence = nullptr;
	if (UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimSequenceBase))
	{
		if (AnimMontage->SlotAnimTracks.Num() > 0)
		{
			const float ClampedTime = FMath::Clamp(Time, 0.f, AnimMontage->CalculateSequenceLength());
			if (FAnimSegment* Segment = AnimMontage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(ClampedTime))
			{
				AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference());
			}
		}
	}
	else
	{
		AnimSequence = Cast<UAnimSequence>(AnimSequenceBase);
	}

	if (AnimSequence)
	{
		TGuardValue<bool> ForceRootLockGuard(AnimSequence->bForceRootLock, false);
		UContextualAnimUtilities::ExtractComponentSpacePose(AnimSequenceBase, BoneContainer, Time, bExtractRootMotion, OutPose);
	}
}

FCompactPoseBoneIndex UContextualAnimUtilities::GetCompactPoseBoneIndexFromPose(const FCSPose<FCompactPose>& Pose, const FName& BoneName)
{
	const FBoneContainer& BoneContainer = Pose.GetPose().GetBoneContainer();
	for (int32 Idx = Pose.GetPose().GetNumBones() - 1; Idx >= 0; Idx--)
	{
		if (BoneContainer.GetReferenceSkeleton().GetBoneName(BoneContainer.GetBoneIndicesArray()[Idx]) == BoneName)
		{
			return FCompactPoseBoneIndex(Idx);
		}
	}

	checkf(false, TEXT("BoneName: %s Pose.Asset: %s Pose.NumBones: %d"), *BoneName.ToString(), *GetNameSafe(Pose.GetPose().GetBoneContainer().GetAsset()), Pose.GetPose().GetNumBones());
	return FCompactPoseBoneIndex(INDEX_NONE);
}

void UContextualAnimUtilities::GenerateIKTargetTrack(const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FContextualAnimAlignmentTrackContainer& Container)
{
	// @TODO: Optimize
	// We are generating IK Target tracks for the duration of the animation and for all the sets in the section
	// We should generate those tracks only for the animations with IK windows and only for the range of that window

	FMemMark Mark(FMemStack::Get());

	const FName& Role = InBindings.GetRoleFromBinding(InBinding);
	const TArray<FContextualAnimIKTargetDefinition>& IKTargetDefs = SceneAsset.GetIKTargetDefsForRole(Role).IKTargetDefs;
	
	if (IKTargetDefs.Num() == 0)
	{
		return;
	}

	if (UAnimSequenceBase* Animation = InBinding.GetAnimation())
	{
		UE_LOGF(LogContextualAnim, Log, "Generating IK Target Tracks. Animation: %ls", *GetNameSafe(Animation));

		const float SampleInterval = 1.f / SceneAsset.GetSampleRate();

		TArray<FBoneIndexType> RequiredBoneIndexArray;

		// Helper structure to group pose extraction per target so we can extract the pose for all the bones that are relative to the same target in one pass
		struct FPoseExtractionHelper
		{
			//const FContextualAnimTrack* TargetAnimTrackPtr = nullptr;
			const FContextualAnimSceneBinding* TargetBindingPtr = nullptr;
			TArray<TTuple<FName, FName, int32, FName, int32>> BonesData; //0: GoalName, 1: MyBoneName 2: MyBoneIndex, 3: TargetBoneName, 4: TargetBoneIndex
		};

		TMap<FName, FPoseExtractionHelper> PoseExtractionHelperMap;
		PoseExtractionHelperMap.Reserve(IKTargetDefs.Num());


		int32 TotalTracks = 0;
		for (const FContextualAnimIKTargetDefinition& IKTargetDef : IKTargetDefs)
		{
			if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Autogenerated)
			{
				const FName TargetRole = IKTargetDef.TargetRoleName;

				FPoseExtractionHelper* DataPtr = PoseExtractionHelperMap.Find(TargetRole);
				if (DataPtr == nullptr)
				{
					// Find track for target role. 
					const FContextualAnimSceneBinding* TargetBinding = InBindings.FindBindingByRole(TargetRole);
					
					DataPtr = &PoseExtractionHelperMap.Add(TargetRole);
					DataPtr->TargetBindingPtr = TargetBinding;
				}

				const FName BoneName = IKTargetDef.BoneName;
				const int32 BoneIndex = Animation->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(BoneName);
				if (BoneIndex == INDEX_NONE)
				{
					UE_LOGF(LogContextualAnim, Warning, "\t Can't find BoneIndex. BoneName: %ls Animation: %ls Skel: %ls",
						*BoneName.ToString(), *GetNameSafe(Animation), *GetNameSafe(Animation->GetSkeleton()));

					continue;
				}

				// Find TargetBoneIndex. Note that we add TargetBoneIndex even if it is INDEX_NONE. In this case, my bone will be relative to the origin of the target actor. 
				// This is to support cases where the target actor doesn't have animation or TargetBoneName is None
				FName TargetBoneName = IKTargetDef.TargetBoneName;
				const UAnimSequenceBase* TargetAnimation = DataPtr->TargetBindingPtr->GetAnimation();

				const int32 TargetBoneIndex = TargetAnimation ? TargetAnimation->GetSkeleton()->GetReferenceSkeleton().FindBoneIndex(TargetBoneName) : INDEX_NONE;
				if (TargetBoneIndex == INDEX_NONE)
				{
					UE_LOGF(LogContextualAnim, Log, "\t Can't find TargetBoneIndex. BoneName: %ls Animation: %ls Skel: %ls. Track for this bone will be relative to the origin of the target role.",
						*TargetBoneName.ToString(), *GetNameSafe(TargetAnimation), TargetAnimation ? *GetNameSafe(TargetAnimation->GetSkeleton()) : nullptr);

					TargetBoneName = NAME_None;
				}

				RequiredBoneIndexArray.AddUnique(BoneIndex);

				DataPtr->BonesData.Add(MakeTuple(IKTargetDef.GoalName, BoneName, BoneIndex, TargetBoneName, TargetBoneIndex));
				TotalTracks++;

				UE_LOGF(LogContextualAnim, Log, "\t Bone added for extraction. GoalName: %ls BoneName: %ls (%d) TargetRole: %ls TargetAnimation: %ls TargetBone: %ls (%d)",
					*IKTargetDef.GoalName.ToString(), *BoneName.ToString(), BoneIndex, *TargetRole.ToString(), *GetNameSafe(TargetAnimation), *TargetBoneName.ToString(), TargetBoneIndex);
			}
		}

		if (TotalTracks > 0)
		{
			// Complete bones chain and create bone container to extract pose from my animation
			Animation->GetSkeleton()->GetReferenceSkeleton().EnsureParentsExistAndSort(RequiredBoneIndexArray);
			FBoneContainer BoneContainer = FBoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *Animation->GetSkeleton());

			// Initialize track container
			Container.Initialize(TotalTracks, SampleInterval);
			Container.Tracks.TrackNames.AddZeroed(TotalTracks);
			Container.Tracks.AnimationTracks.AddZeroed(TotalTracks);

			float Time = 0.f;
			float EndTime = Animation->GetPlayLength();
			int32 SampleIndex = 0;
			while (Time < EndTime)
			{
				Time = FMath::Clamp(SampleIndex * SampleInterval, 0.f, EndTime);
				SampleIndex++;

				// Extract pose from my animation
				FCSPose<FCompactPose> ComponentSpacePose;
				UContextualAnimUtilities::ExtractPoseIgnoringForceRootLock(Animation, BoneContainer, Time, false, ComponentSpacePose);

				// For each target role
				for (auto& Data : PoseExtractionHelperMap)
				{
					// Extract pose from target animation if any
					FCSPose<FCompactPose> OtherComponentSpacePose;
					TArray<FBoneIndexType> OtherRequiredBoneIndexArray;
					FBoneContainer OtherBoneContainer;
					UAnimSequenceBase* OtherAnimation = Data.Value.TargetBindingPtr->GetAnimation();
					if (OtherAnimation)
					{
						// Prepare array with the indices of the bones to extract from target animation
						OtherRequiredBoneIndexArray.Reserve(Data.Value.BonesData.Num());
						for (int32 Idx = 0; Idx < Data.Value.BonesData.Num(); Idx++)
						{
							const int32 TargetBoneIndex = Data.Value.BonesData[Idx].Get<4>();
							if (TargetBoneIndex != INDEX_NONE)
							{
								OtherRequiredBoneIndexArray.AddUnique(TargetBoneIndex);
							}
						}

						if (OtherRequiredBoneIndexArray.Num() > 0)
						{
							// Complete bones chain and create bone container to extract pose form the target animation
							OtherAnimation->GetSkeleton()->GetReferenceSkeleton().EnsureParentsExistAndSort(OtherRequiredBoneIndexArray);
							OtherBoneContainer = FBoneContainer(OtherRequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::DisallowAll), *OtherAnimation->GetSkeleton());

							// Extract pose from target animation
							UContextualAnimUtilities::ExtractPoseIgnoringForceRootLock(OtherAnimation, OtherBoneContainer, Time, false, OtherComponentSpacePose);
						}
					}

					for (int32 Idx = 0; Idx < Data.Value.BonesData.Num(); Idx++)
					{
						const FName TrackName = Data.Value.BonesData[Idx].Get<0>();
						Container.Tracks.TrackNames[Idx] = TrackName;

						// Get bone transform from my animation
						const FName BoneName = Data.Value.BonesData[Idx].Get<1>();
						const FCompactPoseBoneIndex BoneIndex = UContextualAnimUtilities::GetCompactPoseBoneIndexFromPose(ComponentSpacePose, BoneName);
						const FTransform BoneTransform = (ComponentSpacePose.GetComponentSpaceTransform(BoneIndex) * InBinding.GetMeshToScene());

						// Get bone transform from target animation
						FTransform OtherBoneTransform = Data.Value.TargetBindingPtr->GetMeshToScene();
						const FName TargetBoneName = Data.Value.BonesData[Idx].Get<3>();
						if (TargetBoneName != NAME_None)
						{
							const FCompactPoseBoneIndex OtherBoneIndex = UContextualAnimUtilities::GetCompactPoseBoneIndexFromPose(OtherComponentSpacePose, TargetBoneName);
							OtherBoneTransform = (OtherComponentSpacePose.GetComponentSpaceTransform(OtherBoneIndex) * Data.Value.TargetBindingPtr->GetMeshToScene());
						}

						// Get transform relative to target
						const FTransform BoneRelativeToOther = BoneTransform.GetRelativeTransform(OtherBoneTransform);

						// Add transform to the track
						FRawAnimSequenceTrack& NewTrack = Container.Tracks.AnimationTracks[Idx];
						NewTrack.PosKeys.Add(FVector3f(BoneRelativeToOther.GetLocation()));
						NewTrack.RotKeys.Add(FQuat4f(BoneRelativeToOther.GetRotation()));

						UE_LOGF(LogContextualAnim, Verbose, "\t\t Animation: %ls Time: %f BoneName: %ls (T: %ls) Target Animation: %ls TargetBoneName: %ls (T: %ls)",
							*GetNameSafe(Animation), Time, *BoneName.ToString(), *BoneTransform.GetLocation().ToString(),
							*GetNameSafe(OtherAnimation), *TargetBoneName.ToString(), *OtherBoneTransform.GetLocation().ToString());
					}
				}
			}
		}
	}
}

void UContextualAnimUtilities::BP_Montage_GetSectionStartAndEndTime(const UAnimMontage* Montage, int32 SectionIndex, float& OutStartTime, float& OutEndTime)
{
	if(Montage)
	{
		Montage->GetSectionStartAndEndTime(SectionIndex, OutStartTime, OutEndTime);
	}
}

float UContextualAnimUtilities::BP_Montage_GetSectionTimeLeftFromPos(const UAnimMontage* Montage, float Position)
{
	//UAnimMontage::GetSectionTimeLeftFromPos is not const :(
	return Montage ? (const_cast<UAnimMontage*>(Montage))->GetSectionTimeLeftFromPos(Position) : -1.f;
}

float UContextualAnimUtilities::BP_Montage_GetSectionLength(const UAnimMontage* Montage, int32 SectionIndex)
{
	return Montage ? Montage->GetSectionLength(SectionIndex) : -1.f;
}

void UContextualAnimUtilities::DrawSector(FPrimitiveDrawInterface& PDI, const FVector& Origin, const FVector& Direction, float MinDistance, float MaxDistance, float MinAngle, float MaxAngle, const FLinearColor& Color, uint8 DepthPriority, float Thickness, bool bDashedLine)
{
	if(MinAngle == 0 && MaxAngle == 0)
	{
		DrawCircle(&PDI, Origin, FVector(1, 0, 0), FVector(0, 1, 0), Color, 30.f, 12, SDPG_World, 1.f);
		return;
	}

	// Draw Cone lines
	const FVector LeftDirection = Direction.RotateAngleAxis(MinAngle, FVector::UpVector);
	const FVector RightDirection = Direction.RotateAngleAxis(MaxAngle, FVector::UpVector);

	if(bDashedLine)
	{
		DrawDashedLine(&PDI, Origin + (LeftDirection * MinDistance), Origin + (LeftDirection * MaxDistance), Color, 10.f, DepthPriority);
		DrawDashedLine(&PDI, Origin + (RightDirection * MinDistance), Origin + (RightDirection * MaxDistance), Color, 10.f, DepthPriority);
	}
	else
	{
		PDI.DrawLine(Origin + (LeftDirection * MinDistance), Origin + (LeftDirection * MaxDistance), Color, DepthPriority, Thickness);
		PDI.DrawLine(Origin + (RightDirection * MinDistance), Origin + (RightDirection * MaxDistance), Color, DepthPriority, Thickness);
	}

	// Draw Near Arc
	FVector LastDirection = LeftDirection;
	float Angle = MinAngle;
	while (Angle < MaxAngle)
	{
		Angle = FMath::Clamp<float>(Angle + 10, MinAngle, MaxAngle);

		const float Length = MinDistance;
		const FVector NewDirection = Direction.RotateAngleAxis(Angle, FVector::UpVector);
		const FVector LineStart = Origin + (LastDirection * Length);
		const FVector LineEnd = Origin + (NewDirection * Length);
		
		if (bDashedLine)
		{
			DrawDashedLine(&PDI, LineStart, LineEnd, Color, 10.f, DepthPriority);
		}
		else
		{
			PDI.DrawLine(LineStart, LineEnd, Color, DepthPriority, Thickness);
		}
		
		LastDirection = NewDirection;
	}

	// Draw Far Arc
	LastDirection = LeftDirection;
	Angle = MinAngle;
	while (Angle < MaxAngle)
	{
		Angle = FMath::Clamp<float>(Angle + 10, MinAngle, MaxAngle);

		const float Length = MaxDistance;
		const FVector NewDirection = Direction.RotateAngleAxis(Angle, FVector::UpVector);
		const FVector LineStart = Origin + (LastDirection * Length);
		const FVector LineEnd = Origin + (NewDirection * Length);

		if (bDashedLine)
		{
			DrawDashedLine(&PDI, LineStart, LineEnd, Color, 10.f, DepthPriority);
		}
		else
		{
			PDI.DrawLine(LineStart, LineEnd, Color, DepthPriority, Thickness);
		}

		LastDirection = NewDirection;
	}
}

bool UContextualAnimUtilities::BP_CreateContextualAnimSceneBindings(const UContextualAnimSceneAsset* SceneAsset, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings)
{
	if(SceneAsset == nullptr || !SceneAsset->HasValidData())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_CreateContextualAnimSceneBindings Failed. Reason: Invalid or Empty SceneAsset (%ls)", *GetNameSafe(SceneAsset));
		return false;
	}

	const int32 SectionIdx = 0; // Always start from the first section
	return FContextualAnimSceneBindings::TryCreateBindings(*SceneAsset, SectionIdx, Params, OutBindings);
}

bool UContextualAnimUtilities::BP_CreateContextualAnimSceneBindingsForTwoActors(const UContextualAnimSceneAsset* SceneAsset, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings)
{
	if (SceneAsset == nullptr || !SceneAsset->HasValidData())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_CreateContextualAnimSceneBindingsForTwoActors Failed. Reason: Invalid or Empty SceneAsset (%ls)", *GetNameSafe(SceneAsset));
		return false;
	}

	const int32 SectionIdx = 0; // Always start from the first section
	return FContextualAnimSceneBindings::TryCreateBindings(*SceneAsset, SectionIdx, Primary, Secondary, OutBindings);
}

void UContextualAnimUtilities::BP_AddContextualAnimationOverrideProvider(FContextualAnimSceneBindings& InBindings, TScriptInterface<IContextualAnimOverrideInterface>& OverrideProvider, FContextualAnimSceneBindings& OutBindings)
{
	InBindings.SetOverrideProvider(OverrideProvider);
	OutBindings = InBindings;
}

// SceneBindings Blueprint Interface
//------------------------------------------------------------------------------------------

void UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetIndices(const FContextualAnimSceneBindings& Bindings, int32& SectionIdx, int32& AnimSetIdx)
{
	// DEPRECATED: Fail and warn the user if called on bindings with actors in different sections, until removed.

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetIndices Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	if (!Bindings.AreAllBindingsInTheSameSection())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetIndices Failed. Reason: Not all the actors are in the same section. SceneAsset: %ls", 
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneBinding* PrimaryBinding = Bindings.GetPrimaryBinding();
	if (PrimaryBinding == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetIndices Failed. Reason: Can't find Primary Binding. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	SectionIdx = PrimaryBinding->GetSectionIdx();
	AnimSetIdx = PrimaryBinding->GetAnimSetIdx();
}

const FContextualAnimSceneBinding& UContextualAnimUtilities::BP_SceneBindings_GetBindingByRole(const FContextualAnimSceneBindings& Bindings, FName Role)
{
	if(const FContextualAnimSceneBinding* SceneActorData = Bindings.FindBindingByRole(Role))
	{
		return *SceneActorData;
	}

	return FContextualAnimSceneBinding::InvalidBinding;
}

const FContextualAnimSceneBinding& UContextualAnimUtilities::BP_SceneBindings_GetBindingByActor(const FContextualAnimSceneBindings& Bindings, const AActor* Actor)
{
	if (const FContextualAnimSceneBinding* SceneActorData = Bindings.FindBindingByActor(Actor))
	{
		return *SceneActorData;
	}

	return FContextualAnimSceneBinding::InvalidBinding;
}

const FContextualAnimSceneBinding& UContextualAnimUtilities::BP_SceneBindings_GetPrimaryBinding(const FContextualAnimSceneBindings& Bindings)
{
	if (const FContextualAnimSceneBinding* SceneActorData = Bindings.GetPrimaryBinding())
	{
		return *SceneActorData;
	}

	return FContextualAnimSceneBinding::InvalidBinding;
}

void UContextualAnimUtilities::BP_SceneBindings_CalculateWarpPoints(const FContextualAnimSceneBindings& Bindings, TArray<FContextualAnimWarpPoint>& OutWarpPoints)
{
	// DEPRECATED: Fail and warn the user if called on bindings with actors in different sections, until removed

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_CalculateWarpPoints Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	if (!Bindings.AreAllBindingsInTheSameSection())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_CalculateWarpPoints Failed. Reason: Not all the actors are in the same section. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneBinding* PrimaryBinding = Bindings.GetPrimaryBinding();
	if (PrimaryBinding == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_CalculateWarpPoints Failed. Reason: Can't find Primary Binding. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	BP_SceneBindings_CalculateWarpPointsForSectionAtIndex(Bindings, PrimaryBinding->GetSectionIdx(), OutWarpPoints);
}

void UContextualAnimUtilities::BP_SceneBindings_CalculateWarpPointsForSectionAtIndex(const FContextualAnimSceneBindings& Bindings, int32 SectionIdx, TArray<FContextualAnimWarpPoint>& OutWarpPoints)
{
	Bindings.CalculateWarpPoints(OutWarpPoints, SectionIdx);
}

void UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings(const FContextualAnimSceneBindings& Bindings)
{
	// DEPRECATED: Fail and warn the user if called on bindings with actors in different sections, until removed

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	if (!Bindings.AreAllBindingsInTheSameSection())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings Failed. Reason: Not all the actors are in the same section. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneBinding* PrimaryBinding = Bindings.GetPrimaryBinding();
	if (PrimaryBinding == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings Failed. Reason: Can't find Primary Binding. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	BP_SceneBindings_AddOrUpdateWarpTargetsForSectionAtIndex(Bindings, PrimaryBinding->GetSectionIdx());
}

void UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForSectionAtIndex(const FContextualAnimSceneBindings& Bindings, int32 SectionIdx)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneSection* Section = Bindings.GetSceneAsset()->GetSection(SectionIdx);
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_AddOrUpdateWarpTargetsForBindings Failed. Reason: Invalid Section Idx. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	if (Section->GetWarpPointDefinitions().Num() > 0)
	{
		for (const FContextualAnimWarpPointDefinition& WarpPointDef : Section->GetWarpPointDefinitions())
		{
			FContextualAnimWarpPoint WarpPoint;
			if (Bindings.CalculateWarpPoint(WarpPointDef, WarpPoint))
			{
				for (const FContextualAnimSceneBinding& Binding : Bindings)
				{
					if (UMotionWarpingComponent* MotionWarpComp = Binding.GetActor()->FindComponentByClass<UMotionWarpingComponent>())
					{
						const float Time = UContextualAnimUtilities::GetSyncTimeForWarpSection(Binding.GetAnimation(), WarpPointDef.WarpTargetName);
						const FTransform TransformRelativeToWarpPoint = Bindings.GetAlignmentTransformFromBinding(Binding, WarpPointDef.WarpTargetName, Time);
						const FTransform WarpTargetTransform = TransformRelativeToWarpPoint * WarpPoint.Transform;
						MotionWarpComp->AddOrUpdateWarpTargetFromTransform(WarpPoint.Name, WarpTargetTransform);
					}
				}
			}
		}
	}
}

FTransform UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole(const FContextualAnimSceneBindings& Bindings, FName Role, FName RelativeToRole, float Time = 0.f)
{
	FTransform Result = FTransform::Identity;

	if(const UContextualAnimSceneAsset* SceneAsset = Bindings.GetSceneAsset())
	{
		const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByRole(Role);
		if (Binding == nullptr)
		{
			UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole Failed. Reason: Can't find binding for '%ls'.", *Role.ToString());
			return Result;
		}

		const FContextualAnimSceneBinding* OtherBinding = Bindings.FindBindingByRole(RelativeToRole);
		if (OtherBinding == nullptr)
		{
			UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole Failed. Reason: Can't find binding for '%ls'.", *RelativeToRole.ToString());
			return Result;
		}

		if (Binding->GetSectionIdx() != OtherBinding->GetSectionIdx())
		{
			UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole Failed. Reason: Actors in different Sections.");
			return Result;
		}

		if (Binding->GetAnimSetIdx() != OtherBinding->GetAnimSetIdx())
		{
			UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole Failed. Reason: Actors in different Sets.");
			return Result;
		}

		Result = SceneAsset->GetAlignmentTransformForRoleRelativeToOtherRole(Binding->GetSectionIdx(), Binding->GetAnimSetIdx(), Role, RelativeToRole, Time);
	}

	return Result;
}

FTransform UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformForRoleRelativeToWarpPoint(const FContextualAnimSceneBindings& Bindings, FName Role, const FContextualAnimWarpPoint& WarpPoint, float Time)
{
	FTransform Result = FTransform::Identity;

	if (const UContextualAnimSceneAsset* SceneAsset = Bindings.GetSceneAsset())
	{
		if(const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByRole(Role))
		{
			const FContextualAnimTrack& AnimTrack = Bindings.GetAnimTrackFromBinding(*Binding);
			return SceneAsset->GetAlignmentTransform(AnimTrack, WarpPoint.Name, Time);
		}
	}

	return Result;
}

const UAnimSequenceBase* UContextualAnimUtilities::BP_SceneBinding_GetAnimationFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBinding_GetAnimationFromBinding Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return nullptr;
	}

	return Binding.GetAnimation();
}

FName UContextualAnimUtilities::BP_SceneBinding_GetRoleFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBinding_GetRoleFromBinding Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return NAME_None;
	}

	return Bindings.GetAnimTrackFromBinding(Binding).Role;
}

FTransform UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding, const FContextualAnimWarpPoint& WarpPoint)
{
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetAlignmentTransformFromBinding Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return FTransform::Identity;
	}

	const FContextualAnimTrack& AnimTrack = Bindings.GetAnimTrackFromBinding(Binding);

	float StartTime, EndTime;
	AnimTrack.GetStartAndEndTimeForWarpSection(WarpPoint.Name, StartTime, EndTime);

	return Bindings.GetSceneAsset()->GetAlignmentTransform(AnimTrack, WarpPoint.Name, EndTime) * WarpPoint.Transform;
}

void UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNames(const FContextualAnimSceneBindings& Bindings, FName& SectionName, FName& AnimSetName)
{
	// DEPRECATED: Fail and warn the user if called on bindings with actors in different sections, until removed

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNames Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	if (!Bindings.AreAllBindingsInTheSameSection())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNames Failed. Reason: Not all the actors are in the same section. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneBinding* PrimaryBinding = Bindings.GetPrimaryBinding();
	if (PrimaryBinding == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNames Failed. Reason: Can't find Primary Binding. SceneAsset: %ls",
			*GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding(Bindings, *PrimaryBinding, SectionName, AnimSetName);
}

void UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding, FName& SectionName, FName& AnimSetName)
{
	SectionName = NAME_None;
	AnimSetName = NAME_None;

	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding Failed. Reason: Invalid Bindings. SceneAsset: %ls", *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSceneSection* Section = Bindings.GetSceneAsset()->GetSection(Binding.GetSectionIdx());
	if (Section == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding Failed. Reason: '%d' is not a valid section idx in '%ls'",
			Binding.GetSectionIdx(), *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	const FContextualAnimSet* AnimSet = Section->GetAnimSet(Binding.GetAnimSetIdx());
	if (AnimSet == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding Failed. Reason: '%d' is not a valid anim set idx in '%ls'",
			Binding.GetAnimSetIdx(), *GetNameSafe(Bindings.GetSceneAsset()));
		return;
	}

	SectionName = Section->GetName();
	AnimSetName = AnimSet->Name;
}

void UContextualAnimUtilities::BP_SceneBindingContext_GetCurrentSectionAndAnimSetNames(const FContextualAnimSceneBindingContext& BindingContext, FName& SectionName, FName& AnimSetName)
{
	SectionName = NAME_None;
	AnimSetName = NAME_None;

	const UContextualAnimSceneActorComponent* SceneComp = BindingContext.GetSceneActorComponent();
	if (SceneComp == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindingContext_GetCurrentSectionAndAnimSetNames Failed. Reason: Missing SceneActorComp. Actor: %ls", *GetNameSafe(BindingContext.GetActor()));
		return;
	}

	const FContextualAnimSceneBindings& Bindings = SceneComp->GetBindings();
	if (!Bindings.IsValid())
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindingContext_GetCurrentSectionAndAnimSetNames Failed. Reason: Invalid Bindings. Actor: %ls", *GetNameSafe(BindingContext.GetActor()));
		return;
	}

	const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByActor(BindingContext.GetActor());
	if (Binding == nullptr)
	{
		UE_LOGF(LogContextualAnim, Warning, "UContextualAnimUtilities::BP_SceneBindingContext_GetCurrentSectionAndAnimSetNames Failed. Reason: Can't find Binding. Actor: %ls", *GetNameSafe(BindingContext.GetActor()));
		return;
	}

	BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding(Bindings, *Binding, SectionName, AnimSetName);
}

void UContextualAnimUtilities::BP_SceneBinding_GetSectionAndAnimSetIndices(const FContextualAnimSceneBinding& Binding, int32& SectionIdx, int32& AnimSetIdx)
{
	SectionIdx = Binding.GetSectionIdx();
	AnimSetIdx = Binding.GetAnimSetIdx();
}

void UContextualAnimUtilities::GenerateAlignmentTracksForBindings(UContextualAnimSceneAsset& SceneAsset, FContextualAnimSceneBindings& InBindings)
{
	// Necessary for FCompactPose that uses a FAnimStackAllocator (TMemStackAllocator) which allocates from FMemStack.
	// When allocating memory from FMemStack we need to explicitly use FMemMark to ensure items are freed when the scope exits. 
	// UWorld::Tick adds a FMemMark to catch any allocation inside the game tick 
	// but any allocation from outside the game tick (like here when generating the alignment tracks off-line) must explicitly add a mark to avoid a leak 
	FMemMark Mark(FMemStack::Get());

	for (FContextualAnimSceneBinding& Binding : InBindings)
	{
		const FName Role = InBindings.GetRoleFromBinding(Binding);

		GenerateAlignmentTrackForBinding(SceneAsset, Role, Binding);
	}
}

void UContextualAnimUtilities::GenerateAlignmentTrackForBinding(UContextualAnimSceneAsset& SceneAsset, const FName& Role, FContextualAnimSceneBinding& Binding)
{
	const FContextualAnimSceneSection& Section = *SceneAsset.GetSection(Binding.GetSectionIdx());
	const FContextualAnimSet& AnimSet = *Section.GetAnimSet(Binding.GetAnimSetIdx());

	const FTransform MeshToComponentInverse = FTransform(SceneAsset.GetMeshToComponentForRole(Role).GetRotation()).Inverse();
	const float SampleInterval = 1.f / SceneAsset.GetSampleRate();

	// Initialize tracks for each alignment section
	const int32 TotalTracks = Section.GetWarpPointDefinitions().Num();

	FContextualAnimAlignmentTrackContainer& AlignmentData = Binding.GetAlignmentData();
	AlignmentData.Initialize(TotalTracks, SampleInterval);

	for (int32 Idx = 0; Idx < TotalTracks; Idx++)
	{
		AlignmentData.Tracks.TrackNames.Add(Section.GetWarpPointDefinitions()[Idx].WarpTargetName);
		AlignmentData.Tracks.AnimationTracks.AddZeroed();
	}

	if (const UAnimSequenceBase* Animation = Binding.GetAnimation())
	{
		float Time = 0.f;
		float EndTime = Animation->GetPlayLength();
		int32 SampleIndex = 0;
		while (Time < EndTime)
		{
			Time = FMath::Clamp(SampleIndex * SampleInterval, 0.f, EndTime);
			SampleIndex++;

			const FTransform RootTransform = MeshToComponentInverse * (UContextualAnimUtilities::ExtractRootTransformFromAnimation(Animation, Time) * Binding.GetMeshToScene());

			for (int32 Idx = 0; Idx < TotalTracks; Idx++)
			{
				FRawAnimSequenceTrack& AlignmentTrack = AlignmentData.Tracks.AnimationTracks[Idx];

				const FTransform* WarpPointTransform = AnimSet.WarpPoints.Find(Section.GetWarpPointDefinitions()[Idx].WarpTargetName);
				const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

				AlignmentTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
				AlignmentTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
			}
		}
	}
	else
	{
		const FTransform RootTransform = MeshToComponentInverse * Binding.GetMeshToScene();

		for (int32 Idx = 0; Idx < TotalTracks; Idx++)
		{
			FRawAnimSequenceTrack& SceneTrack = AlignmentData.Tracks.AnimationTracks[Idx];

			const FTransform* WarpPointTransform = AnimSet.WarpPoints.Find(Section.GetWarpPointDefinitions()[Idx].WarpTargetName);
			const FTransform RootRelativeToWarpPoint = WarpPointTransform ? RootTransform.GetRelativeTransform(*WarpPointTransform) : RootTransform;

			SceneTrack.PosKeys.Add(FVector3f(RootRelativeToWarpPoint.GetLocation()));
			SceneTrack.RotKeys.Add(FQuat4f(RootRelativeToWarpPoint.GetRotation()));
		}
	}
}
