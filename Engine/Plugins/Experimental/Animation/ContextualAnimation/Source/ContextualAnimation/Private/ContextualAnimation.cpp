// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimation.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "ContextualAnimationModule"

void FContextualAnimationModule::StartupModule()
{
}

void FContextualAnimationModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FContextualAnimationModule, ContextualAnimation)
