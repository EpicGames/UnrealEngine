// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AnimNotifyState_EarlyOutContextualAnimWindow.generated.h"

#define UE_API CONTEXTUALANIMATION_API

/** Notify used to allow player to early out from a contextual anim. Usually used at the end of the animations to improve responsivess */
UCLASS(MinimalAPI, meta = (DisplayName = "Early Out Contextual Anim"))
class UAnimNotifyState_EarlyOutContextualAnimWindow : public UAnimNotifyState
{
	GENERATED_BODY()

public:

	/** Whether to force all the actors in the interaction to exit the interaction or only us */
	UPROPERTY(EditAnywhere, Category = Config)
	bool bStopEveryone = false;

	UE_API UAnimNotifyState_EarlyOutContextualAnimWindow(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void NotifyTick(USkeletalMeshComponent* MeshComp, UAnimSequenceBase* Animation, float FrameDeltaTime, const FAnimNotifyEventReference& EventReference) override;
	UE_API virtual FString GetNotifyName_Implementation() const override;
};

#undef UE_API
