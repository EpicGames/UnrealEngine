// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "AlphaBlend.h"
#include "AnimNotifyState_IKWindow.generated.h"

#define UE_API CONTEXTUALANIMATION_API

/** AnimNotifyState used to define areas in an animation where IK should be enable */
UCLASS(MinimalAPI, meta = (DisplayName = "IK Window"))
class UAnimNotifyState_IKWindow : public UAnimNotifyState
{
	GENERATED_BODY()

public:

	virtual void Serialize(FArchive& Ar) override;

	UPROPERTY(EditAnywhere, Category = Config)
	FName GoalName;

	UPROPERTY(EditAnywhere, Category = Config)
	FAlphaBlend BlendIn;

	UPROPERTY(EditAnywhere, Category = Config)
	FAlphaBlend BlendOut;

	UPROPERTY()
	bool bEnable = true;

	UE_API UAnimNotifyState_IKWindow(const FObjectInitializer& ObjectInitializer);

	UE_API virtual FString GetNotifyName_Implementation() const override;

	static UE_API float GetIKAlphaValue(const FName& GoalName, const struct FAnimMontageInstance* MontageInstance);
};

#undef UE_API
