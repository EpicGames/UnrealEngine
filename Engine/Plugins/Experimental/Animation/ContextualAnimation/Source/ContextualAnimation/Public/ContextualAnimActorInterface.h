// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ContextualAnimActorInterface.generated.h"

UINTERFACE()
class UContextualAnimActorInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IContextualAnimActorInterface : public IInterface
{
	GENERATED_IINTERFACE_BODY()

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Contextual Animation System")
	class USkeletalMeshComponent* GetMesh() const;

	/** Called from CAS Editor on the preview actor after spawn. Useful to give that actor a chance to run game-specific logic needed to be usable as preview actor in the CAS Editor */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Contextual Animation System")
	CONTEXTUALANIMATION_API void RefreshEditorPreview();
};
