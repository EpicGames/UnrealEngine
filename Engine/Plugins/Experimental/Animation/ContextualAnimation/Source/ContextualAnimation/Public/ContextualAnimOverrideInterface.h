// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "ContextualAnimTypes.h"

#include "ContextualAnimOverrideInterface.generated.h"

class UAnimSequenceBase;
struct FContextualAnimActorPreviewData;


USTRUCT(BlueprintType)
struct FContextualAnimationOverrideData
{
	GENERATED_BODY()

	FContextualAnimationOverrideData()
	{

	}

	FContextualAnimationOverrideData(const FContextualAnimTrack& InAnimTrack)
	{
		AnimSequence = InAnimTrack.Animation;
		MeshToScene = InAnimTrack.MeshToScene;
	}
	
	UPROPERTY(EditAnywhere, Category = Overrides)
	TObjectPtr<UAnimSequenceBase> AnimSequence = nullptr;

	UPROPERTY(EditAnywhere, Category = Overrides)
	FTransform MeshToScene = FTransform::Identity;

	/** Container for alignment tracks */
	UPROPERTY()
	FContextualAnimAlignmentTrackContainer AlignmentData;

	/** Container for auto generate IK Target Tracks */
	UPROPERTY()
	FContextualAnimAlignmentTrackContainer IKTargetData;
};

UINTERFACE(BlueprintType)
class UContextualAnimOverrideInterface : public UInterface
{
	GENERATED_BODY()
};

class IContextualAnimOverrideInterface : public IInterface
{
	GENERATED_BODY()

public:
	virtual bool GetCASAnimationDataOverride(const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, const FContextualAnimTrack& InAnimTrack, FContextualAnimationOverrideData& OutData) = 0;

#if WITH_EDITORONLY_DATA
	/**
	 * Called during SpawnPreviewActor to allow the provider to override the preview data for a given role.
	 * Return true and populate OutPreviewData to replace whatever is set in the scene asset's OverridePreviewData.
	 */
	virtual bool GetCASAnimActorPreviewDataOverrideForRole(const FName& InRole, FContextualAnimActorPreviewData& OutPreviewData) { return false; }

	/**
	 * Called after a preview actor has been spawned, allowing post-spawn setup (e.g. applying cosmetics).
	 */
	virtual void OnCASPreviewActorSpawned(const FName& InRole, AActor* InSpawnedActor) {}
#endif

};
