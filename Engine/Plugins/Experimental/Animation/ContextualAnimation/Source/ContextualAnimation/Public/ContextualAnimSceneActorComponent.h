// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorComponents/IKRigInterface.h"
#include "Components/ActorComponent.h"
#include "ContextualAnimTypes.h"

#include "ContextualAnimSceneActorComponent.generated.h"

#define UE_API CONTEXTUALANIMATION_API

class AActor;
class FPrimitiveSceneProxy;
class UAnimInstance;
class UAnimMontage;
class UAnimSequenceBase;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FContextualAnimSceneActorCompDelegate, class UContextualAnimSceneActorComponent*, SceneActorComponent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FContextualAnimPlayMontageNotifyBeginDelegate, class UContextualAnimSceneActorComponent*, SceneActorComponent, FName, NotifyName);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FContextualAnimOnMontageBlendingOutDelegate, class UContextualAnimSceneActorComponent*, SceneComponent, UAnimMontage* const, BlendingOutMontage, bool, bInterrupted);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FContextualAnimOnMontageReachedEndDelegate, class UContextualAnimSceneActorComponent*, SceneComponent, UAnimMontage* const, Montage, int32, LoopCounter);

USTRUCT(BlueprintType)
struct FContextualAnimWarpTarget
{
	GENERATED_BODY()

	FContextualAnimWarpTarget() = default;
	FContextualAnimWarpTarget(const FName InRole, const FName InWarpTargetName, const FTransform& InTargetTransform)
		: Role(InRole)
		, TargetName(InWarpTargetName)
		, TargetLocation(InTargetTransform.GetLocation())
		, TargetRotation(InTargetTransform.GetRotation())
	{
	}

	UPROPERTY(EditAnywhere, Category = "Default")
	FName Role = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Default")
	FName TargetName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Default")
	FVector TargetLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = "Default")
	FQuat TargetRotation = FQuat::Identity;
};

/** Base struct for replicated data with a rep counter */
USTRUCT()
struct FContextualAnimRepData
{
	GENERATED_BODY()

	/** Auto increment counter to ensure replication even if the data is the same */
	UPROPERTY()
	uint8 RepCounter = 0;

	void IncrementRepCounter();
	bool IsValid() const { return RepCounter != 0; }
};

/** Used to replicate start/stop contextual anim events */
USTRUCT()
struct FContextualAnimRepBindingsData : public FContextualAnimRepData
{
	GENERATED_BODY()

	UPROPERTY()
	FContextualAnimSceneBindings Bindings;

	UPROPERTY()
	TArray<FContextualAnimWarpPoint> WarpPoints;

	UPROPERTY()
	TArray<FContextualAnimWarpTarget> ExternalWarpTargets;

	UPROPERTY()
	float StartTime = 0.f;

	void Reset()
	{
		RepCounter = 0;
		Bindings.Reset();
		WarpPoints.Reset();
		ExternalWarpTargets.Reset();
		StartTime = 0.f;
	}
};

/** Used to replicate a late join event */
USTRUCT()
struct FContextualAnimRepLateJoinData : public FContextualAnimRepData
{
	GENERATED_BODY()

	/** Actor that is joining the interaction */
	UPROPERTY()
	TWeakObjectPtr<AActor> Actor = nullptr;

	/** Role in the interaction the actor is gonna play */
	UPROPERTY()
	FName Role = NAME_None;

	UPROPERTY()
	TArray<FContextualAnimWarpPoint> WarpPoints;

	UPROPERTY()
	TArray<FContextualAnimWarpTarget> ExternalWarpTargets;

	void Reset()
	{
		RepCounter = 0;
		Actor = nullptr;
		Role = NAME_None;
		WarpPoints.Reset();
		ExternalWarpTargets.Reset();
	}
};

/** Used to transition events */
USTRUCT()
struct FContextualAnimRepTransitionData : public FContextualAnimRepData
{
	GENERATED_BODY()

	UPROPERTY()
	uint8 Id = 0;

	UPROPERTY()
	uint8 SectionIdx = 0;

	UPROPERTY()
	uint8 AnimSetIdx = 0;

	UPROPERTY()
	bool bStopEveryone = false;

	UPROPERTY()
	TArray<FContextualAnimWarpPoint> WarpPoints;

	UPROPERTY()
	TArray<FContextualAnimWarpTarget> ExternalWarpTargets;

	void Reset()
	{
		RepCounter = 0;
		Id = 0;
		SectionIdx = 0;
		AnimSetIdx = 0;
		bStopEveryone = false;
		WarpPoints.Reset();
		ExternalWarpTargets.Reset();
	}
};

UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent))
class UContextualAnimSceneActorComponent : public UActorComponent, public IIKGoalCreatorInterface
{
	GENERATED_BODY()

public:

	/** Event that happens when the actor owner of this component joins an scene */
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FContextualAnimSceneActorCompDelegate OnJoinedSceneDelegate;

	/** Event that happens when the actor owner of this component leave an scene */
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FContextualAnimSceneActorCompDelegate OnLeftSceneDelegate;

	UPROPERTY(BlueprintAssignable, Category = "Events")
	FContextualAnimPlayMontageNotifyBeginDelegate OnPlayMontageNotifyBeginDelegate;

	UPROPERTY(BlueprintAssignable, Category = "Events")
	FContextualAnimOnMontageBlendingOutDelegate OnMontageBlendingOutDelegate;

	/** Delegate called when the currently playing montage reaches the end but does not blends out */
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FContextualAnimOnMontageReachedEndDelegate OnMontageReachedEndDelegate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	TObjectPtr<class UContextualAnimSceneAsset> SceneAsset;

	/** Optional offset from the actor transform. Can be used to modify the final world space transform used for alignment. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Config")
	FTransform AlignmentOffset;

	UE_API UContextualAnimSceneActorComponent(const FObjectInitializer& ObjectInitializer);

	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	UE_API virtual void GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;

	UE_API virtual void AddIKGoals_Implementation(TMap<FName, FIKRigGoal>& OutGoals) override;

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Actor Component")
	const FContextualAnimSceneBindings& GetBindings() const { return Bindings; };

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool IsInActiveScene() const;

	/** Called when the actor owner of this component joins an scene */
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API void OnJoinedScene(const FContextualAnimSceneBindings& InBindings);

	/** Called from the scene instance when the actor owner of this component leave an scene */
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API void OnLeftScene();

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Actor Component")
	const TArray<FContextualAnimIKTarget>& GetIKTargets() const { return IKTargets; }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Actor Component")
	UE_API const FContextualAnimIKTarget& GetIKTargetByGoalName(FName GoalName) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool StartContextualAnimScene(const FContextualAnimSceneBindings& InBindings);
	
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool LateJoinContextualAnimScene(AActor* Actor, FName Role);
	
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool TransitionContextualAnimScene(FName SectionName);
	
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool TransitionContextualAnimSceneToSpecificSet(FName SectionName, int32 AnimSetIdx);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API bool TransitionSingleActor(int32 SectionIdx, int32 AnimSetIdx);

	UE_API bool StartContextualAnimScene(const FContextualAnimSceneBindings& InBindings, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);
	UE_API bool LateJoinContextualAnimScene(AActor* Actor, FName Role, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);
	UE_API bool TransitionContextualAnimScene(FName SectionName, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);
	UE_API bool TransitionContextualAnimScene(FName SectionName, int32 AnimSetIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);
	UE_API bool TransitionSingleActor(int32 SectionIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);
	UE_API bool TransitionSingleActor(int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Actor Component")
	UE_API void EarlyOutContextualAnimScene(bool bStopEveryone = false);

	UE_API bool IsOwnerLocallyControlled() const;

	UFUNCTION()
	UE_API void OnIgnoredActorEndPlay(AActor* Actor, EEndPlayReason::Type EndPlayReason);

protected:

	/** Cache of the owner's AnimInstance for easy access. Only valid while an interaction is active */
	UPROPERTY(Transient)
	TWeakObjectPtr<UAnimInstance> OwnerAnimInstance = nullptr;

	/** 
	 * Replicated copy of the bindings so we can start the action on simulated proxies 
	 * This gets replicated only from the initiator of the action and then set on all the other members of the interaction
	 */
	UPROPERTY(Transient, ReplicatedUsing = OnRep_Bindings)
	FContextualAnimRepBindingsData RepBindings;

	UPROPERTY(Transient, ReplicatedUsing = OnRep_LateJoinData)
	FContextualAnimRepLateJoinData RepLateJoinData;

	UPROPERTY(Transient, ReplicatedUsing = OnRep_TransitionData)
	FContextualAnimRepTransitionData RepTransitionData;

	UPROPERTY(Transient, ReplicatedUsing = OnRep_RepTransitionSingleActor)
	FContextualAnimRepTransitionData RepTransitionSingleActorData;

	/**
	 * Bindings for the interaction we are currently playing.
	 * Used to update IK, keep montage in sync, disable/enable collision between actors etc
	 */
	UPROPERTY(Transient)
	FContextualAnimSceneBindings Bindings;

	/** List of IKTarget for this frame */
	UPROPERTY(Transient)
	TArray<FContextualAnimIKTarget> IKTargets;

	/** 
	 * Motion warping doesn't remove warp targets when a warping window ends, so we keep track of the warp targets added during the interaction 
	 * and remove them at the end of it to prevent issues caused by dangling warp targets after an interaction.
	 */
	UPROPERTY(Transient)
	TArray<FName> WarpTargetNamesCache;

	struct FCharacterRotationProperties
	{
		bool bAllowPhysicsRotationDuringAnimRootMotion = false;
		bool bUseControllerDesiredRotation = false;
		bool bOrientRotationToMovement = false;
	};

	struct FCharacterProperties
	{
		bool bIgnoreClientMovementErrorChecksAndCorrection = false;
		bool bReplicatingMovement = false;
		bool bSimulatePhysics = false;
		TOptional<FCharacterRotationProperties> RotationProperties;
		TOptional<EMovementMode> MovementMode;
		uint8 CustomMovementMode = 0;
		TArray<TTuple<ECollisionChannel, ECollisionResponse>> CollisionResponses;
		TOptional<struct FCollisionResponseContainer> CollisionProfileResponses;
	};
	FCharacterProperties CharacterPropertiesBackup;

	UE_API void UpdateIKTargets();

	/** 
	 * Event called right before owner's mesh ticks the pose when we are in a scene instance and IK Targets are required. 
	 * Used to update IK Targets before animation need them 
	 */
	UFUNCTION()
	UE_API virtual void OnTickPose(class USkinnedMeshComponent* SkinnedMeshComponent, float DeltaTime, bool bNeedsValidRootMotion);

	UFUNCTION()
	UE_API virtual void OnRep_Bindings();

	UFUNCTION()
	UE_API void OnRep_LateJoinData();

	UFUNCTION()
	UE_API void OnRep_RepTransitionSingleActor();

	UFUNCTION()
	UE_API void OnRep_TransitionData();

	UE_API void SetIgnoreCollisionWithActor(AActor& Actor, bool bValue) const;
	UE_API void SetIgnoreCollisionWithOtherActors(bool bValue) const;
	
	UE_API virtual void SetCollisionState(const FContextualAnimSceneBinding& Binding);
	UE_API virtual void RestoreCollisionState(const FContextualAnimSceneBinding& Binding);

	UE_API void SetMovementState(const FContextualAnimSceneBinding& Binding, const FContextualAnimTrack& AnimTrack);
	UE_API void RestoreMovementState(const FContextualAnimSceneBinding& Binding);

	UFUNCTION()
	UE_API void OnMontageBlendingOut(UAnimMontage* Montage, bool bInterrupted);

	UFUNCTION()
	UE_API void OnPlayMontageNotifyBegin(FName NotifyName, const FBranchingPointNotifyPayload& BranchingPointNotifyPayload);

	// @TODO: These two functions are going to replace OnJoinedScene and OnLeftScene
	// main different is that these new functions are taking care of animation playback too

	UE_API void JoinScene(const FContextualAnimSceneBindings& InBindings, const TArray<FContextualAnimWarpPoint> WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets, float StartTime = 0.f);

	UE_API virtual void OnJoinScene(const FContextualAnimSceneBinding& Binding);

	UE_API void LeaveScene();

	UE_API virtual bool CanLeaveScene(const FContextualAnimSceneBinding& Binding);

	// Called just before the scene leaving happens (which calls OnLeaveScene)
	UE_API virtual void OnPreLeaveScene(const FContextualAnimSceneBinding& Binding);

	UE_API virtual void OnLeaveScene(const FContextualAnimSceneBinding& Binding);

	UE_API void LateJoinScene(const FContextualAnimSceneBindings& InBindings, int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);

	UE_API virtual void OnLateJoinScene(const FContextualAnimSceneBinding& Binding, int32 SectionIdx, int32 AnimSetIdx);

	UFUNCTION(Server, Reliable, WithValidation)
	UE_API void ServerStartContextualAnimScene(const FContextualAnimSceneBindings& InBindings);

	UFUNCTION(Server, Reliable, WithValidation)
	UE_API void ServerEarlyOutContextualAnimScene(bool bStopEveryone);

	UE_API virtual void PlayAnimation_Internal(UAnimSequenceBase* Animation, float StartTime, bool bSyncPlaybackTime);

	UE_API virtual void AddOrUpdateWarpTargets(int32 SectionIdx, int32 AnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);

	UE_API void HandleTransitionSelf(int32 NewSectionIdx, int32 NewAnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);

	UE_API void HandleTransitionEveryone(int32 NewSectionIdx, int32 NewAnimSetIdx, const TArray<FContextualAnimWarpPoint>& WarpPoints, const TArray<FContextualAnimWarpTarget>& ExternalWarpTargets);

	UE_API virtual void OnTransitionScene(const FContextualAnimSceneBinding& Binding, int32 SectionIdx, int32 AnimSetIdx);

	UE_API virtual void OnTransitionSingleActor(const FContextualAnimSceneBinding& Binding, int32 SectionIdx, int32 AnimSetIdx);

	UE_API virtual void OtherActorLeftScene(AActor& Actor);

	/** 
	 * Finds ideal start time for the interaction based on the distance from the actor to the warp target in the animation and the actual distance in game. 
	 * Effectively skipping the portion of the 'approach-motion' the actor doesn't need.
     * Returns 0 if the actor is far enough to play the full animation from the beginning.
	 */
	float FindIdealStartTime(const FContextualAnimSceneBindings& InBindings, const FContextualAnimSceneBinding& Binding, const TArray<FContextualAnimWarpPoint>& WarpPoints) const;

private:

	TArray<TWeakObjectPtr<const UAnimMontage>, TInlineAllocator<5>> AnimsPlayed;
	int32 LoopCounter = 0;
};

#undef UE_API
