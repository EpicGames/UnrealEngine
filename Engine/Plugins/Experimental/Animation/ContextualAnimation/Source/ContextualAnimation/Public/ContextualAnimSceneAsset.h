// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CollisionProfile.h"
#include "Engine/DataAsset.h"
#include "ContextualAnimTypes.h"
#include "GameplayTagContainer.h"
#include "ContextualAnimSceneAsset.generated.h"

#define UE_API CONTEXTUALANIMATION_API

class UContextualAnimSceneInstance;
class UContextualAnimSceneAsset;

UENUM(BlueprintType)
enum class EContextualAnimCollisionBehavior : uint8
{
	None,
	IgnoreActorWhenMoving,
	IgnoreChannels,
	CollisionProfile
};

USTRUCT(BlueprintType)
struct FContextualAnimIgnoreChannelsParam
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Role = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	TArray<TEnumAsByte<ECollisionChannel>> Channels;
};

USTRUCT(BlueprintType)
struct FContextualAnimCollisionProfileParam
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Role = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	FCollisionProfileName Profile;
};

USTRUCT(BlueprintType)
struct FContextualAnimCollisionBehaviorParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	EContextualAnimCollisionBehavior CollisionBehavior = EContextualAnimCollisionBehavior::None;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "CollisionBehavior==EContextualAnimCollisionBehavior::IgnoreChannels", EditConditionHides))
	TArray<FContextualAnimIgnoreChannelsParam> CollisionChannelsToIgnoreParams;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "CollisionBehavior==EContextualAnimCollisionBehavior::CollisionProfile", EditConditionHides))
	TArray<FContextualAnimCollisionProfileParam> CollisionProfileParams;

	const TArray<TEnumAsByte<ECollisionChannel>>& GetCollisionChannelsToIgnoreForRole(FName Role) const
	{
		if (CollisionBehavior == EContextualAnimCollisionBehavior::IgnoreChannels)
		{
			if (const FContextualAnimIgnoreChannelsParam* Param = CollisionChannelsToIgnoreParams.FindByPredicate([Role](const FContextualAnimIgnoreChannelsParam& Item) { return Item.Role == Role; }))
			{
				return Param->Channels;
			}
		}

		static const TArray<TEnumAsByte<ECollisionChannel>> EmptyArray;
		return EmptyArray;
	}

	FCollisionProfileName GetCollisionProfileNameForRole(FName Role) const
	{
		if (CollisionBehavior == EContextualAnimCollisionBehavior::CollisionProfile)
		{
			if (const FContextualAnimCollisionProfileParam* Param = CollisionProfileParams.FindByPredicate([Role](const FContextualAnimCollisionProfileParam& Item) { return Item.Role == Role; }))
			{
				return Param->Profile;
			}
		}

		return FCollisionProfileName();
	}

};

USTRUCT()
struct FContextualAnimAttachmentParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Role = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	FName SocketName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	FTransform RelativeTransform = FTransform::Identity;
};

UCLASS(MinimalAPI, Blueprintable)
class UContextualAnimRolesAsset : public UDataAsset
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TArray<FContextualAnimRoleDefinition> Roles;

	UContextualAnimRolesAsset(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {};

	const FContextualAnimRoleDefinition* FindRoleDefinitionByName(const FName& Name) const
	{
		return Roles.FindByPredicate([Name](const FContextualAnimRoleDefinition& RoleDef) { return RoleDef.Name == Name; });
	}

	inline int32 GetNumRoles() const { return Roles.Num(); }
};

/**
 * Contains AnimTracks for each role in the interaction.
 * Example: An specific set for a interaction with a car would have two tracks, one with the animation for the character and another one with the animation for the car.
 * It is common to have variations of the same action with different animations. We could have one AnimSet with the animations for getting into the car from the driver side and another for getting into the car from the passenger side.
*/
USTRUCT(BlueprintType)
struct FContextualAnimSet
{
	GENERATED_BODY()

	/** List of tracks with animation (and relevant data specific to that animation) for each role */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	TArray<FContextualAnimTrack> Tracks;

	/** Map of WarpTargetNames and Transforms for this set. Generated off line based on warp points defined in the asset */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	TMap<FName, FTransform> WarpPoints;

	/** Optional name to identify this set */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	FName Name = NAME_None;

	/** Used by the selection mechanism to 'break the tie' when multiple Sets can be selected */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float RandomWeight = 1.f;

	UE_API int32 GetNumMandatoryRoles() const;
};

/** Named container with one or more ContextualAnimSet */
USTRUCT(BlueprintType)
struct FContextualAnimSceneSection
{
	GENERATED_BODY()

public:

	UE_API const FContextualAnimSet* GetAnimSet(int32 AnimSetIdx) const; 

	UE_API const FContextualAnimTrack* GetAnimTrack(int32 AnimSetIdx, const FName& Role) const;

	UE_API const FContextualAnimTrack* GetAnimTrack(int32 AnimSetIdx, int32 AnimTrackIdx) const;

	UE_API FTransform GetIKTargetTransformForRoleAtTime(int32 AnimSetIdx, FName Role, FName TrackName, float Time) const;

	UE_API const FContextualAnimTrack* FindFirstAnimTrackForRoleThatPassesSelectionCriteria(const FName& Role, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const;

	inline FName GetName() const { return Name; }
	inline const TArray<FContextualAnimWarpPointDefinition>& GetWarpPointDefinitions() const { return WarpPointDefinitions; }
	inline int32 GetNumAnimSets() const { return AnimSets.Num(); }
	inline bool ShouldSyncAnimations() const { return bSyncAnimations; }

protected:

	UPROPERTY(EditAnywhere, Category = "Defaults")
	FName Name;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	TArray<FContextualAnimSet> AnimSets;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (TitleProperty = "WarpTargetName"))
	TArray<FContextualAnimWarpPointDefinition> WarpPointDefinitions;

	UPROPERTY(EditAnywhere, Category = "Defaults")
	bool bSyncAnimations = true;

	UE_API void GenerateAlignmentTracks(UContextualAnimSceneAsset& SceneAsset);
	UE_API void GenerateIKTargetTracks(UContextualAnimSceneAsset& SceneAsset);

 	friend class UContextualAnimSceneAsset;
 	friend class FContextualAnimViewModel;
};

USTRUCT(BlueprintType)
struct FContextualAnimPoint
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FName Role = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	FTransform Transform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	float Speed = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	int32 SectionIdx = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	int32 AnimSetIdx = INDEX_NONE;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	int32 AnimTrackIdx = INDEX_NONE;

	FContextualAnimPoint(){}
	FContextualAnimPoint(const FName& InRole, const FTransform& InTransform, float InSpeed, int32 InSectionIdx, int32 InAnimSetIdx, int32 InAnimTrackIdx)
		: Role(InRole), Transform(InTransform), Speed(InSpeed), SectionIdx(InSectionIdx), AnimSetIdx(InAnimSetIdx), AnimTrackIdx(InAnimTrackIdx)
	{}
};

UENUM(BlueprintType)
enum class EContextualAnimPointType : uint8
{
	FirstFrame,
	SyncFrame,
	LastFrame
};

UENUM(BlueprintType)
enum class EContextualAnimCriterionToConsider : uint8
{
	All,
	Spatial,
	Other
};

#if WITH_EDITORONLY_DATA

UENUM(BlueprintType)
enum class EContextualAnimActorPreviewType : uint8
{
	SkeletalMesh,
	StaticMesh,
	Actor,
	None
};

USTRUCT(BlueprintType)
struct FContextualAnimActorPreviewData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Role;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	EContextualAnimActorPreviewType Type = EContextualAnimActorPreviewType::StaticMesh;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Type==EContextualAnimActorPreviewType::SkeletalMesh", EditConditionHides))
	TSoftObjectPtr<class USkeletalMesh> PreviewSkeletalMesh;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Type==EContextualAnimActorPreviewType::SkeletalMesh", EditConditionHides))
	TSoftClassPtr<class UAnimInstance> PreviewAnimInstance;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Type==EContextualAnimActorPreviewType::StaticMesh", EditConditionHides))
	TSoftObjectPtr<class UStaticMesh> PreviewStaticMesh;

	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Type==EContextualAnimActorPreviewType::Actor", EditConditionHides))
	TSoftClassPtr<class AActor> PreviewActorClass;
};

/** Entry in a preview sequence that defines which section to play and how many times */
USTRUCT(BlueprintType)
struct FContextualAnimPreviewSequenceEntry
{
	GENERATED_BODY()

	/** Which section to play */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (GetOptions = "GetSectionNames"))
	FName SectionName;

	/** How many times to play this section (1 = play once, 2 = repeat twice, etc.) */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (ClampMin = "1", ClampMax = "100"))
	int32 LoopCount = 1;

	/** Which anim set to use. INDEX_NONE = use the editor's active anim set for that section */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	int32 AnimSetIdx = INDEX_NONE;
};

#endif  // WITH_EDITORONLY_DATA

UCLASS(MinimalAPI, Blueprintable)
class UContextualAnimSceneAsset : public UDataAsset
{
	GENERATED_BODY()

public:

	typedef TFunctionRef<UE::ContextualAnim::EForEachResult(const FContextualAnimTrack& AnimTrack)> FForEachAnimTrackFunction;

	UE_API UContextualAnimSceneAsset(const FObjectInitializer& ObjectInitializer);
	
	/** Migrate deprecated objects. */
	UE_API virtual void PostLoad() override;

	UE_API virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	UE_API void PrecomputeData();
	
	UE_API void ForEachAnimTrack(FForEachAnimTrackFunction Function) const;

	inline const FGameplayTag& GetTag() const { return Tag; }
	inline const FName& GetPrimaryRole() const { return PrimaryRole; }
	UE_DEPRECATED(5.9, "Direct access has been deprecated, Please use CollisionBehaviors array version instead")
	inline EContextualAnimCollisionBehavior GetCollisionBehavior() const { return CollisionBehaviors.Num() > 0 ? CollisionBehaviors[0].CollisionBehavior : EContextualAnimCollisionBehavior::None; }

	inline const TArray<FContextualAnimCollisionBehaviorParams>&  GetCollisionBehaviors() const { return CollisionBehaviors; }
	inline int32 GetSampleRate() const { return SampleRate; }
	inline float GetRadius() const { return Radius; }
	inline bool ShouldPrecomputeAlignmentTracks() const { return bPrecomputeAlignmentTracks; }
	inline bool ShouldIgnoreClientMovementErrorChecksAndCorrection() const { return bIgnoreClientMovementErrorChecksAndCorrection; }
	inline bool ShouldDisableMovementReplicationForSimulatedProxy() const { return bDisableMovementReplicationForSimulatedProxy; }
	inline bool ShouldFindIdealAnimStartTime() const { return bFindIdealAnimStartTime; }
	inline float GetFindIdealStartTimeMargin() const { return FindIdealStartTimeMargin; }

	const TArray<FContextualAnimAttachmentParams>& GetAttachmentParams() const { return AttachmentParams; }

	const FContextualAnimAttachmentParams* GetAttachmentParamsForRole(FName Role) const
	{
		return AttachmentParams.FindByPredicate([Role](const FContextualAnimAttachmentParams& Item) { return Item.Role == Role; });
	}

	const FContextualAnimIKTargetParams& GetIKTargetParams() const { return IKTargetParams; }

	bool HasValidData() const { return RolesAsset != nullptr && Sections.Num() > 0 && Sections[0].AnimSets.Num() > 0; }

	const UContextualAnimRolesAsset* GetRolesAsset() const { return RolesAsset; }

	UFUNCTION()
	UE_API TArray<FName> GetRoles() const;

	int32 GetNumRoles() const { return RolesAsset ? RolesAsset->GetNumRoles() : 0; }

	UE_API int32 GetNumMandatoryRoles(int32 SectionIdx, int32 AnimSetIdx) const;

	UE_API const FTransform& GetMeshToComponentForRole(const FName& Role) const;

	UFUNCTION()
	UE_API TArray<FName> GetSectionNames() const;

	UE_API int32 GetNumSections() const;

	UE_API int32 GetNumAnimSetsInSection(int32 SectionIdx) const;

	UE_API const FContextualAnimSceneSection* GetSection(int32 SectionIdx) const;

	UE_API const FContextualAnimSceneSection* GetSection(const FName& SectionName) const;

	UE_API const FContextualAnimSet* GetAnimSet(int32 SectionIdx, int32 AnimSetIdx) const;

	UE_API int32 GetSectionIndex(const FName& SectionName) const;
	
 	UE_API const FContextualAnimTrack* GetAnimTrack(int32 SectionIdx, int32 AnimSetIdx, const FName& Role) const;
 
 	UE_API const FContextualAnimTrack* GetAnimTrack(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx) const;

	UE_API FTransform GetIKTargetTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, const FName& TrackName, float Time) const;
	UE_API FTransform GetIKTargetTransform(const FContextualAnimSceneBinding& InBinding, const FName& TrackName, float Time) const;

	UE_API FTransform GetAlignmentTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, int32 WarpPointIdx, float Time) const;
	UE_API FTransform GetAlignmentTransform(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx, const FName& WarpPointName, float Time) const;
	UE_API FTransform GetAlignmentTransform(const FContextualAnimTrack& AnimTrack, int32 WarpPointIdx, float Time) const;
	UE_API FTransform GetAlignmentTransform(const FContextualAnimTrack& AnimTrack, const FName& WarpPointName, float Time) const;
	UE_API FTransform GetAlignmentTransform(const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FName WarpPointName, float Time) const;
	UE_API FTransform GetAlignmentTransformForRoleRelativeToOtherRole(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName OtherRole, float Time) const;

	UE_API const FContextualAnimTrack* FindAnimTrackForRoleWithClosestEntryLocation(int32 SectionIdx, const FName& Role, const FContextualAnimSceneBindingContext& Primary, const FVector& TestLocation) const;

	UE_API const FContextualAnimTrack* FindAnimTrackByAnimation(const UAnimSequenceBase* Animation) const;

	UE_API const FContextualAnimIKTargetDefContainer& GetIKTargetDefsForRole(const FName& Role) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset")
	UE_API void GetAlignmentPointsForSecondaryRole(EContextualAnimPointType Type, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, TArray<FContextualAnimPoint>& OutResult) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset")
	UE_API void GetAlignmentPointsForSecondaryRoleConsideringSelectionCriteria(EContextualAnimPointType Type, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier, EContextualAnimCriterionToConsider CriterionToConsider, TArray<FContextualAnimPoint>& OutResult) const;

public:

	// Blueprint Interface
	//------------------------------------------------------------------------------------------

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Find Animation For Role"))
	UE_API UAnimSequenceBase* BP_FindAnimationForRole(int32 SectionIdx, int32 AnimSetIdx, FName Role) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Find AnimSet Index By Animation"))
	UE_API int32 BP_FindAnimSetIndexByAnimation(int32 SectionIdx, const UAnimSequenceBase* Animation) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Get Alignment Transform For Role Relative To WarpPoint"))
	UE_API FTransform BP_GetAlignmentTransformForRoleRelativeToWarpPoint(int32 SectionIdx, int32 AnimSetIdx, FName Role, float Time) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Get IK Target Transform For Role At Time"))
	UE_API FTransform BP_GetIKTargetTransformForRoleAtTime(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName TrackName, float Time) const;
	
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Get IK Target Transform For Binding At Time"))
	UE_API FTransform BP_GetIKTargetTransformForBindingAtTime(const FContextualAnimSceneBinding& InBinding, FName TrackName, float Time) const;

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Get Start and End Time For Warp Section"))
	UE_API void BP_GetStartAndEndTimeForWarpSection(int32 SectionIdx, int32 AnimSetIdx, FName Role, FName WarpSectionName, float& OutStartTime, float& OutEndTime) const;

	//@TODO: Kept around only to do not break existing content. It will go away in the future.
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset")
	UE_API bool Query(FName Role, FContextualAnimQueryResult& OutResult, const FContextualAnimQueryParams& QueryParams, const FTransform& ToWorldTransform) const;
	UE_API float FindBestAnimStartTime(const FContextualAnimTrack& AnimTrack, const FVector& LocalLocation) const;

protected:

	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<UContextualAnimRolesAsset> RolesAsset;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (GetOptions = "GetRoles"))
	FName PrimaryRole = NAME_None;

	/** Optional GameplayTag that can be used to identify this particular scene asset */
	UPROPERTY(EditAnywhere, Category = "Settings")
	FGameplayTag Tag;

#if WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Preview", meta = (TitleProperty = "Role"))
	TArray<FContextualAnimActorPreviewData> OverridePreviewData;

	UPROPERTY(EditAnywhere, Category = "Preview", meta = (TitleProperty = "Role", MustImplement = "/Script/ContextualAnimation.ContextualAnimOverrideInterface"))
	TSubclassOf<UObject> PreviewOverrideProvider = nullptr;;

	/** Defines a sequence of sections for multi-section preview (e.g. Intro -> Loop x3 -> Outro) */
	UPROPERTY(EditAnywhere, Category = "Preview", meta = (TitleProperty = "SectionName"))
	TArray<FContextualAnimPreviewSequenceEntry> MultiSectionPreviewSequence;

#endif // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Defaults")
	TArray<FContextualAnimSceneSection> Sections;

	UPROPERTY(EditAnywhere, Category = "Settings")
	float Radius = 0.f;

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Direct access has been deprecated, Please use CollisionBehaviors array version instead"))
	EContextualAnimCollisionBehavior CollisionBehavior_DEPRECATED = EContextualAnimCollisionBehavior::None;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Direct access has been deprecated, Please use CollisionBehaviors array version instead"))
	TArray<FContextualAnimIgnoreChannelsParam> CollisionChannelsToIgnoreParams_DEPRECATED;
#endif  // WITH_EDITORONLY_DATA

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FContextualAnimCollisionBehaviorParams> CollisionBehaviors;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TArray<FContextualAnimAttachmentParams> AttachmentParams;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (DisplayName = "IK Target Params"))
	FContextualAnimIKTargetParams IKTargetParams;

	/** 
	 * Whether to find ideal start time for the interaction based on the distance from the actor to the warp target in the animation and the actual distance in game.
	 * Effectively skipping the portion of the 'approach-motion' the actor doesn't need. 
	 */
	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bFindIdealAnimStartTime = false;

	/**
	 * Time (in seconds) reserved at the end of the warp window to ensure motion warping has enough room to reach the target.
	 * The ideal start time search will not go beyond (WarpEndTime - FindIdealStartTimeMargin).
	 */ 
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditCondition = "bFindIdealAnimStartTime", EditConditionHides))
	float FindIdealStartTimeMargin = 0.2f;

	/** Whether to ignore movement error checks and corrections during the interaction for player characters */
	UPROPERTY(EditAnywhere, Category = "Settings", AdvancedDisplay)
	bool bIgnoreClientMovementErrorChecksAndCorrection = true;

	/** Whether to disable movement replication during the interaction for simulated proxies (NPCs only). */
	UPROPERTY(EditAnywhere, Category = "Settings", AdvancedDisplay)
	bool bDisableMovementReplicationForSimulatedProxy = false;

	/** Whether we should extract and cache alignment tracks off line. */
	UPROPERTY(EditAnywhere, Category = "Settings", AdvancedDisplay)
	bool bPrecomputeAlignmentTracks = false;

	/** Sample rate (frames per second) used when sampling the animations to generate alignment and IK tracks */
	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "1", ClampMax = "60"), AdvancedDisplay)
	int32 SampleRate = 15;

 	friend class FContextualAnimViewModel;
};

#undef UE_API
