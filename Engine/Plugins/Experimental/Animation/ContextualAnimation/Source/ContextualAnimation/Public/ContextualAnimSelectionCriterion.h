// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContextualAnimSelectionCriterion.generated.h"

#define UE_API CONTEXTUALANIMATION_API

class UContextualAnimSceneAsset;
struct FContextualAnimSceneBindingContext;

UENUM(BlueprintType)
enum class EContextualAnimCriterionType : uint8
{
	Spatial,
	Other
};

// UContextualAnimSelectionCriterion
//===========================================================================

UCLASS(MinimalAPI, Abstract, BlueprintType, EditInlineNew)
class UContextualAnimSelectionCriterion : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Default")
	EContextualAnimCriterionType Type = EContextualAnimCriterionType::Spatial;

	UE_API UContextualAnimSelectionCriterion(const FObjectInitializer& ObjectInitializer);

	UE_API class UContextualAnimSceneAsset* GetSceneAssetOwner() const;

	virtual bool DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const { return false; }
};

// UContextualAnimSelectionCriterion_Blueprint
//===========================================================================

UCLASS(MinimalAPI, Abstract, Blueprintable)
class UContextualAnimSelectionCriterion_Blueprint : public UContextualAnimSelectionCriterion
{
	GENERATED_BODY()

public:

	UE_API UContextualAnimSelectionCriterion_Blueprint(const FObjectInitializer& ObjectInitializer);

	UFUNCTION(BlueprintImplementableEvent, Category = "Default", meta = (DisplayName = "Does Querier Pass Condition"))
	UE_API bool BP_DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const;

	UFUNCTION(BlueprintPure, Category = "Default")
	UE_API const UContextualAnimSceneAsset* GetSceneAsset() const;

	UE_API virtual bool DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const override;
};

// UContextualAnimSelectionCriterion_TriggerArea
//===========================================================================

UCLASS(MinimalAPI)
class UContextualAnimSelectionCriterion_TriggerArea : public UContextualAnimSelectionCriterion
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = "Default", meta = (EditFixedOrder))
	TArray<FVector> PolygonPoints;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "0", UIMin = "0"))
	float Height = 100.f;

	UE_API UContextualAnimSelectionCriterion_TriggerArea(const FObjectInitializer& ObjectInitializer);

	UE_API virtual bool DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const override;
};

// UContextualAnimSelectionCriterion_Cone
//===========================================================================

UENUM(BlueprintType)
enum class EContextualAnimCriterionConeMode : uint8
{
	/** Uses the angle between the vector from querier to primary and querier forward vector rotated by offset */
	ToPrimary,

	/** Uses the angle between the vector from primary to querier and primary forward vector rotated by offset */
	FromPrimary
};

UCLASS(MinimalAPI)
class UContextualAnimSelectionCriterion_Cone : public UContextualAnimSelectionCriterion
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	EContextualAnimCriterionConeMode Mode = EContextualAnimCriterionConeMode::ToPrimary;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "0", UIMin = "0"))
	float Distance = 200.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "0", UIMin = "0", ClampMax = "180", UIMax = "180"))
	float HalfAngle = 45.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "-180", UIMin = "-180", ClampMax = "180", UIMax = "180"))
	float Offset = 0.f;

	UContextualAnimSelectionCriterion_Cone(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}

	UE_API virtual bool DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const override;
};

// UContextualAnimSelectionCriterion_Distance
//===========================================================================

UENUM(BlueprintType)
enum class EContextualAnimCriterionDistanceMode : uint8
{
	Distance_3D,
	Distance_2D
};

UCLASS(MinimalAPI)
class UContextualAnimSelectionCriterion_Distance : public UContextualAnimSelectionCriterion
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default")
	EContextualAnimCriterionDistanceMode Mode = EContextualAnimCriterionDistanceMode::Distance_2D;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "0", UIMin = "0"))
	float MinDistance = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Default", meta = (ClampMin = "0", UIMin = "0"))
	float MaxDistance = 0.f;

	UContextualAnimSelectionCriterion_Distance(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}

	UE_API virtual bool DoesQuerierPassCondition(const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Querier) const override;
};

#undef UE_API
