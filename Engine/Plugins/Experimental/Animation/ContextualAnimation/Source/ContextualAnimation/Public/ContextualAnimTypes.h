// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimSequence.h"
#include "GameFramework/Actor.h"
#include "GameplayTagContainer.h"
#include "ContextualAnimTypes.generated.h"

#define UE_API CONTEXTUALANIMATION_API

struct FContextualAnimSceneBindingContext;

CONTEXTUALANIMATION_API DECLARE_LOG_CATEGORY_EXTERN(LogContextualAnim, Log, All);

class AActor;
class UAnimInstance;
class UAnimSequenceBase;
class UContextualAnimSelectionCriterion;
class UContextualAnimSceneAsset;
class UContextualAnimSceneActorComponent;
class UCharacterMovementComponent;
class UMotionWarpingComponent;
class IContextualAnimOverrideInterface;

struct FAnimMontageInstance;
struct FContextualAnimationOverrideData;

namespace UE 
{
	namespace ContextualAnim 
	{
		enum class EForEachResult : uint8
		{
			Break,
			Continue,
		};
	}
}

/** Container for alignment tracks */
USTRUCT()
struct FContextualAnimAlignmentTrackContainer
{
	GENERATED_BODY()

	UPROPERTY()
	FAnimSequenceTrackContainer Tracks;

	UPROPERTY()
	float SampleInterval = 0.f;

	void Initialize(int32 InNumTracks, float InSampleInterval)
	{
		Tracks.AnimationTracks.Empty(InNumTracks);
		Tracks.TrackNames.Empty(InNumTracks);
		SampleInterval = InSampleInterval;
	}

	void Empty()
	{
		Tracks.AnimationTracks.Empty();
		Tracks.TrackNames.Empty();
		SampleInterval = 0.f;
	}

	UE_API FTransform ExtractTransformAtTime(int32 TrackIndex, float Time) const;
	UE_API FTransform ExtractTransformAtTime(const FName& TrackName, float Time) const;
};

USTRUCT(BlueprintType)
struct FContextualAnimTrack
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TObjectPtr<UAnimSequenceBase> Animation = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float AnimMaxStartTime = 0.f;

	/** If true, will set the movement mode of the character using CAS, to 'MovementMode', during the scope of this anim track. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bChangeMovementMode = true;

	/** If 'bChangeMovementMode' is true, will set the movement mode of the character using CAS, to this value, during the scope of this anim track. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (EditConditionHides, EditCondition = "bChangeMovementMode"))
	TEnumAsByte<enum EMovementMode> MovementMode = EMovementMode::MOVE_Walking;

	/** If 'bChangeMovementMode' is true, and the 'MovementMode' is Custom, it'll set this custom movement mode during the scope of this anim track. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (EditConditionHides, EditCondition = "MovementMode==EMovementMode::MOVE_Custom"))
	uint8 CustomMovementMode = 0;

	/** If true: disable orientation of the character towards movement, controller's rotation and physics driven rotation during root motion. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bControlCharacterRotation = true;

	/** Whether the actor that should play this animation is optional */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bOptional = false;

	/** Container for alignment tracks */
	UPROPERTY()
	FContextualAnimAlignmentTrackContainer AlignmentData;

	/** Container for auto generate IK Target Tracks */
	UPROPERTY()
	FContextualAnimAlignmentTrackContainer IKTargetData;

	UPROPERTY(EditAnywhere, Instanced, Category = "Defaults")
	TArray<TObjectPtr<UContextualAnimSelectionCriterion>> SelectionCriteria;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FTransform MeshToScene;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Defaults")
	FName Role = NAME_None;

	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	int32 SectionIdx = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	int32 AnimSetIdx = INDEX_NONE;

	UPROPERTY(BlueprintReadOnly, Category = "Defaults")
	int32 AnimTrackIdx = INDEX_NONE;

	// DEPRECATED: Will go away soon
	UE_API float GetSyncTimeForWarpSection(int32 WarpSectionIndex) const;
	UE_API float GetSyncTimeForWarpSection(const FName& WarpSectionName) const;

	UE_API void GetStartAndEndTimeForWarpSection(int32 WarpSectionIndex, float& OutStartTime, float& OutEndTime) const;
	UE_API void GetStartAndEndTimeForWarpSection(const FName& WarpSectionName, float& OutStartTime, float& OutEndTime) const;

	UE_API bool DoesQuerierPassSelectionCriteria(const FContextualAnimSceneBindingContext& PrimaryActorData, const FContextualAnimSceneBindingContext& QuerierData) const;

	UE_API FTransform GetRootTransformAtTime(float Time) const;

	static UE_API const FContextualAnimTrack EmptyTrack;
};

/** Defines when the actor should start playing the animation */
UENUM(BlueprintType)
enum class EContextualAnimJoinRule : uint8
{
	Default,
	Late
};

UENUM(BlueprintType)
enum class EContextualAnimIKTargetProvider : uint8
{
	/** 
	 * IK Targets are auto generated from the animations 
	 * @see: UContextualAnimSceneAssetBase::GenerateIKTargetTracks
	 */
	Autogenerated,

	/** IK Target defined by a bone or a socket in the animation */
	Bone,

	//@TODO: Add collision query methods
};

USTRUCT(BlueprintType)
struct FContextualAnimIKTargetDefinition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName GoalName = NAME_None;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	EContextualAnimIKTargetProvider Provider = EContextualAnimIKTargetProvider::Autogenerated;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName TargetRoleName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName TargetBoneName = NAME_None;

	bool operator==(const FContextualAnimIKTargetDefinition& Other) const 
	{
		return  GoalName == Other.GoalName && 
				BoneName == Other.BoneName && 
				Provider == Other.Provider && 
				TargetRoleName == Other.TargetRoleName &&
				TargetBoneName == Other.TargetBoneName;
	}
};

USTRUCT(BlueprintType)
struct FContextualAnimIKTarget
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName GoalName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	float Alpha = 0.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FTransform Transform;

	FContextualAnimIKTarget() {}
	FContextualAnimIKTarget(const FName& InGoalName, const FName& InBoneName, float InAlpha, const FTransform& InTransform)
		: GoalName(InGoalName), BoneName(InBoneName), Alpha(InAlpha), Transform(InTransform) {}

	static UE_API const FContextualAnimIKTarget InvalidIKTarget;
};

USTRUCT(BlueprintType)
struct FContextualAnimIKTargetDefContainer
{
	GENERATED_BODY()

	/** Role this IK Target definitions are for */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Role = NAME_None;

	/** List of IK Target definitions */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (DisplayName = "Definitions"))
	TArray<FContextualAnimIKTargetDefinition> IKTargetDefs;

	static UE_API const FContextualAnimIKTargetDefContainer EmptyContainer;
};

UENUM(BlueprintType)
enum class EContextualAnimIKTargetAlphaProvider : uint8
{	
	/** 
	 * Alpha value is determine by an IK Window in the animation. 
	 * IK Target transform will only be calculated when alpha is bigger than 0 
	 */
	AnimNotifyState,

	/** 
	 * Alpha value is determined by a curve in the animation with the same name as the IK Target
	 * IK Target transform will only be calculated when alpha is bigger than 0 
	 */
	Curve,

	/** IK Target transform will be calculated during the entire interaction. Alpha value will be always 1 */
	None
};

USTRUCT()
struct FContextualAnimIKTargetParams
{
	GENERATED_BODY()

	/** IK Target definitions for each role in the interaction */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (TitleProperty = "Role", DisplayName = "IK Target Defs For Each Role"))
	TArray<FContextualAnimIKTargetDefContainer> IKTargetDefsForEachRole;

	/** Method used to determine the alpha of IK targets at runtime */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	EContextualAnimIKTargetAlphaProvider AlphaProvider = EContextualAnimIKTargetAlphaProvider::AnimNotifyState;
};

// FContextualAnimRoleDefinition
///////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType)
struct FContextualAnimRoleDefinition
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bIsCharacter = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (EditCondition = "bIsCharacter", EditConditionHides))
	float PreviewCapsuleHalfHeight = 88.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (EditCondition = "bIsCharacter", EditConditionHides))
	float PreviewCapsuleRadius = 34.f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FTransform MeshToComponent = FTransform(FRotator(0.f, 0.f, 0.f));

	static const FContextualAnimRoleDefinition InvalidRoleDefinition;
};

// EContextualAnimWarpPointDefinitionMode
///////////////////////////////////////////////////////////////////////

/** Different modes for defining warp points */
UENUM(BlueprintType)
enum class EContextualAnimWarpPointDefinitionMode : uint8
{
	/** 
	 * Warp point will be at the location/rotation of the primary actor. 
	 * Commonly used when interacting with static objects 
	*/
	PrimaryActor,

	/** 
	 * Warp point will be at the location/rotation of a socket on the primary actor. 
	 * Commonly used when re-using the same interaction animations to interact with objects with different proportions 
	*/
	Socket,

	/**
	 * Warp point will be calculated based on a set of rules.
	 * Commonly used when we want actors to align at some point between them
	*/
	Custom
};

/** Parameters used to calculate a warp point when using 'Custom' mode */
USTRUCT(BlueprintType)
struct FContextualAnimWarpPointCustomParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (GetOptions = "GetRoles"))
	FName Origin = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bAlongClosestDistance = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (GetOptions = "GetRoles", EditCondition = "bAlongClosestDistance"))
	FName OtherRole = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bAlongClosestDistance"))
	float Weight = 0.f;
};

/**
 * Contain the params necessary to calculate a warp point for the scene. 
 * A warp point is a transform in the scene that can be used as reference to calculate alignment between actors
*/
USTRUCT(BlueprintType)
struct FContextualAnimWarpPointDefinition
{
	GENERATED_BODY()

	/** Name of the warp target in the warping window this warp point is for */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	FName WarpTargetName = NAME_None;

	/** Method used to calculate this warp point */
	UPROPERTY(EditAnywhere, Category = "Defaults")
	EContextualAnimWarpPointDefinitionMode Mode = EContextualAnimWarpPointDefinitionMode::PrimaryActor;

	/** Name of the socket in the primary actor acting as warping point. Only relevant when Mode is Socket */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Mode==EContextualAnimWarpPointDefinitionMode::Socket", EditConditionHides))
	FName SocketName = NAME_None;

	/** Set of rules used to calculate the warp point. Only relevant when Mode is Custom */
	UPROPERTY(EditAnywhere, Category = "Defaults", meta = (EditCondition = "Mode==EContextualAnimWarpPointDefinitionMode::Custom", EditConditionHides, FullyExpand = "true"))
	FContextualAnimWarpPointCustomParams Params;
};

// FContextualAnimWarpPoint
///////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType)
struct FContextualAnimWarpPoint
{
	GENERATED_BODY()

	FContextualAnimWarpPoint() = default;
	FContextualAnimWarpPoint(const FName InName, const FTransform& InTransform) : Name(InName), Transform(InTransform) {}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FName Name = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FTransform Transform;
};

// FContextualAnimSceneBindingContext
///////////////////////////////////////////////////////////////////////

USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/ContextualAnimation.ContextualAnimUtilities:BP_SceneBindingContext_MakeFromActor"))
struct FContextualAnimSceneBindingContext
{
	GENERATED_BODY()

	FContextualAnimSceneBindingContext() {}

	FContextualAnimSceneBindingContext(AActor* InActor, const TOptional<FTransform>& InExternalTransform = TOptional<FTransform>(), const TOptional<FVector>& InExternalVelocity = TOptional<FVector>())
		: Actor(InActor), ExternalTransform(InExternalTransform), ExternalVelocity(InExternalVelocity) {}

	FContextualAnimSceneBindingContext(const FTransform& InExternalTransform, const TOptional<FVector>& InExternalVelocity = TOptional<FVector>())
		: ExternalTransform(InExternalTransform), ExternalVelocity(InExternalVelocity) {}

	FContextualAnimSceneBindingContext(AActor* InActor, const FGameplayTagContainer& InExternalTags)
		: Actor(InActor), ExternalGameplayTags(InExternalTags) {}

	AActor* GetActor() const { return Actor.Get(); }

	UE_API UAnimInstance* GetAnimInstance() const;

	UE_API USkeletalMeshComponent* GetSkeletalMeshComponent() const;

	UE_API UContextualAnimSceneActorComponent* GetSceneActorComponent() const;

	UE_API UCharacterMovementComponent* GetCharacterMovementComponent() const;

	UE_API UMotionWarpingComponent* GetMotionWarpingComponent() const;

	UE_API void SetExternalTransform(const FTransform& InTransform);

	UE_API FTransform GetTransform() const;

	UE_API FVector GetVelocity() const;

	UE_API void AddGameplayTag(const FGameplayTag& Tag);
	
	const FGameplayTagContainer& GetGameplayTags() const { return ExternalGameplayTags; }

	UE_API bool HasMatchingGameplayTag(const FGameplayTag& TagToCheck) const;

	UE_API bool HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const;

	UE_API bool HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const;

private:

	UPROPERTY()
	TWeakObjectPtr<AActor> Actor = nullptr;

	UPROPERTY(NotReplicated)
	mutable TWeakObjectPtr<UContextualAnimSceneActorComponent> CachedSceneActorComp = nullptr;

	UPROPERTY(NotReplicated)
	mutable TWeakObjectPtr<UAnimInstance> CachedAnimInstance = nullptr;

	UPROPERTY(NotReplicated)
	mutable TWeakObjectPtr<USkeletalMeshComponent> CachedSkeletalMesh = nullptr;

	UPROPERTY(NotReplicated)
	mutable TWeakObjectPtr<UCharacterMovementComponent> CachedMovementComp = nullptr;

	UPROPERTY(NotReplicated)
	mutable TWeakObjectPtr<UMotionWarpingComponent> CachedMotionWarpingComp = nullptr;

	TOptional<FTransform> ExternalTransform;

	TOptional<FVector> ExternalVelocity;

	FGameplayTagContainer ExternalGameplayTags;
};

/** Represent an actor bound to a role in the scene */
USTRUCT(BlueprintType)
struct FContextualAnimSceneBinding
{
	GENERATED_BODY()

	FContextualAnimSceneBinding() {}
	UE_API FContextualAnimSceneBinding(const FContextualAnimSceneBindingContext& InContext, const FContextualAnimTrack& InAnimTrack);

	inline const FContextualAnimSceneBindingContext& GetContext() const { return Context; }
	inline FContextualAnimSceneBindingContext& GetContext() { return Context; }
	inline AActor* GetActor() const { return GetContext().GetActor(); }
	inline FTransform GetTransform() const { return GetContext().GetTransform(); }
	inline FVector GetVelocity()  const { return GetContext().GetVelocity(); }
	inline UAnimInstance* GetAnimInstance() const { return Context.GetAnimInstance(); }
	inline USkeletalMeshComponent* GetSkeletalMeshComponent() const { return Context.GetSkeletalMeshComponent(); }
	inline UContextualAnimSceneActorComponent* GetSceneActorComponent() const { return Context.GetSceneActorComponent(); }
	inline UCharacterMovementComponent* GetCharacterMovementComponent() const { return Context.GetCharacterMovementComponent(); }
	inline UMotionWarpingComponent* GetMotionWarpingComponent() const { return Context.GetMotionWarpingComponent(); }
	inline int8 GetSectionIdx() const { return SectionIdx; }
	inline int8 GetAnimSetIdx() const { return AnimSetIdx; }
	inline int8 GetAnimTrackIdx() const { return AnimTrackIdx; }

	UE_API FTransform GetIKTargetTransform(const FName& TrackName, float Time) const;
	UE_API FTransform GetAlignmentTransform(const FName& TrackName, float Time) const;
	
	UE_API void SetAnimTrack(const FContextualAnimTrack& InAnimTrack);

	/** Return the current playback time of the animation this actor is playing */
	UE_API float GetAnimMontageTime() const;

	UE_API FName GetCurrentSection() const;

	UE_API int32 GetCurrentSectionIndex() const;

	/** Returns the ActiveMontageInstance or null in the case of static actors */
	UE_API FAnimMontageInstance* GetAnimMontageInstance() const;

	static UE_API const FContextualAnimSceneBinding InvalidBinding;

	UAnimSequenceBase* GetAnimation() const { return Animation; }

	FContextualAnimAlignmentTrackContainer& GetAlignmentData() { return AlignmentData; }
	const FContextualAnimAlignmentTrackContainer& GetAlignmentData() const { return AlignmentData; }

	FContextualAnimAlignmentTrackContainer& GetIKTargetData() { return IKTargetData; }
	const FContextualAnimAlignmentTrackContainer& GetIKTargetData() const { return IKTargetData; }

	const FTransform& GetMeshToScene() const { return MeshToScene; }
	FTransform& GetMeshToScene() { return MeshToScene; }

	void SetOverrideData(const FContextualAnimationOverrideData& InData);

private:

	friend class UContextualAnimSceneInstance;
	friend struct FContextualAnimSceneBindings;

	UPROPERTY()
	FContextualAnimSceneBindingContext Context;

	UPROPERTY()
	int8 SectionIdx = INDEX_NONE;

	UPROPERTY()
	int8 AnimSetIdx = INDEX_NONE;

	UPROPERTY()
	int8 AnimTrackIdx = INDEX_NONE;

	UPROPERTY()
	TObjectPtr<UAnimSequenceBase> Animation = nullptr;

	/** Container for alignment tracks */
	UPROPERTY(NotReplicated)
	FContextualAnimAlignmentTrackContainer AlignmentData;

	/** Container for auto generate IK Target Tracks */
	UPROPERTY(NotReplicated)
	FContextualAnimAlignmentTrackContainer IKTargetData;
	
	UPROPERTY()
	FTransform MeshToScene;

	friend FContextualAnimSceneBinding;
};

USTRUCT(BlueprintType)
struct FContextualAnimSceneBindings
{
	GENERATED_BODY()

	FContextualAnimSceneBindings(){}
	
	UE_DEPRECATED(5.7, "This constructor is deprecated. Please use the version with the other signature")
	UE_API FContextualAnimSceneBindings(const UContextualAnimSceneAsset& InSceneAsset, int32 InSectionIdx, int32 InAnimSetIdx);

	UE_API FContextualAnimSceneBindings(const UContextualAnimSceneAsset& InSceneAsset);

	const FContextualAnimSceneBinding* FindBindingByActor(const AActor* Actor) const
	{
		return Actor ? Data.FindByPredicate([Actor](const FContextualAnimSceneBinding& Item) { return Item.GetActor() == Actor; }) : nullptr;
	}

	FContextualAnimSceneBinding* FindBindingByActor(const AActor* Actor)
	{
		return Actor ? Data.FindByPredicate([Actor](const FContextualAnimSceneBinding& Item) { return Item.GetActor() == Actor; }) : nullptr;
	}

	const FContextualAnimSceneBinding* FindBindingByRole(const FName& Role) const
	{
		return Role != NAME_None ? Data.FindByPredicate([this, &Role](const FContextualAnimSceneBinding& Item) { return GetAnimTrackFromBinding(Item).Role == Role; }) : nullptr;
	}

	UE_API void AddReferencedObjects(FReferenceCollector& Collector);

	inline uint8 GetID() const { return Id; }
	inline const UContextualAnimSceneAsset* GetSceneAsset() const { return SceneAsset; }

	UE_DEPRECATED(5.7, "This getter and its corresponding property is deprecated. SectionIdx is now stored per Binding")
	inline int32 GetSectionIdx() const { return SectionIdx_DEPRECATED; }

	UE_DEPRECATED(5.7, "This getter and its corresponding property is deprecated. AnimSetIdx is now stored per Binding")
	inline int32 GetAnimSetIdx() const { return AnimSetIdx_DEPRECATED; }

	inline int32 Num() const { return Data.Num(); }
	inline int32 Add(const FContextualAnimSceneBinding& NewData) { return Data.Add(NewData); }
	inline const TArray<FContextualAnimSceneBinding>& GetBindings() const { return Data; }

	inline TArray<FContextualAnimSceneBinding>::RangedForIteratorType      begin() { return Data.begin(); }
	inline TArray<FContextualAnimSceneBinding>::RangedForConstIteratorType begin() const { return Data.begin(); }
	inline TArray<FContextualAnimSceneBinding>::RangedForIteratorType      end() { return Data.end(); }
	inline TArray<FContextualAnimSceneBinding>::RangedForConstIteratorType end() const { return Data.end(); }

	static UE_API bool CheckConditions(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params);
	static UE_API bool TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings);
	static UE_API bool TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings);
	static UE_API bool TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, int32 AnimSetIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings);
	static UE_API bool TryCreateBindings(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings);
	static UE_API int32 FindAnimSet(const UContextualAnimSceneAsset& SceneAsset, int32 SectionIdx, const TMap<FName, FContextualAnimSceneBindingContext>& Params);

	UE_API bool BindActorToRole(AActor& ActorRef, FName Role, int32 SectionIdx, int32 AnimSetIdx);

	UE_API bool SetRoleWarpTarget(const FName Role, const FName WarpTargetName, const FTransform& Transform);

	UE_DEPRECATED(5.7, "This function is deprecated. Please use the version that takes a Section Index")
	UE_API void CalculateWarpPoints(TArray<FContextualAnimWarpPoint>& OutWarpPoints) const;

	UE_API void CalculateWarpPoints(TArray<FContextualAnimWarpPoint>& OutWarpPoints, int32 SectionIdx) const;

	UE_API bool CalculateWarpPoint(const FContextualAnimWarpPointDefinition& WarpPointDef, FContextualAnimWarpPoint& OutWarpPoint) const;

	UE_API const FContextualAnimTrack& GetAnimTrackFromBinding(const FContextualAnimSceneBinding& Binding) const;
	UE_API const FName& GetRoleFromBinding(const FContextualAnimSceneBinding& Binding) const;
	UE_API FTransform GetAlignmentTransformFromBinding(const FContextualAnimSceneBinding& Binding, const FName& TrackName, float Time) const;
	UE_API const FContextualAnimIKTargetDefContainer& GetIKTargetDefContainerFromBinding(const FContextualAnimSceneBinding& Binding) const;
	UE_API FTransform GetIKTargetTransformFromBinding(const FContextualAnimSceneBinding& Binding, const FName& TrackName, float Time) const;

	UE_DEPRECATED(5.7, "This function is deprecated. Please use ShouldBindingSyncAnimation instead")
	UE_API bool ShouldSyncAnimation() const;

	UE_API bool ShouldBindingSyncAnimation(const FContextualAnimSceneBinding& Binding) const;
	UE_API const FContextualAnimSceneBinding* GetSyncLeader() const;
	UE_API const FContextualAnimSceneBinding* GetPrimaryBinding() const;

	UE_API bool IsValid() const;

	UE_API void Reset();

	UE_API void Clear();

	UE_API void GenerateUniqueId();

	UE_API int32 FindAnimSetForTransitionTo(int32 NewSectionIdx) const;
	UE_API void TransitionTo(int32 NewSectionIdx, int32 NewAnimSetIdx);

	UE_API bool RemoveActor(AActor& ActorRef);

	UE_API bool AreAllBindingsInTheSameSection() const;

	UE_API void SetOverrideProvider(const TScriptInterface <IContextualAnimOverrideInterface>& Provider);
	
	const TScriptInterface <IContextualAnimOverrideInterface>& GetOverrideProvider() const { return OverrideProviderInterface;}

	/*
	* Updates Each Bindings Overrides, Including Animation, Alignment IK, ETC 
	* Returns True is there is a valid Scene and Override Provider
	*/
	UE_API bool UpdateBindingOverridesIfNeeded();

private:

	friend class UContextualAnimManager;

	UPROPERTY()
	uint8 Id = 0;

	UPROPERTY()
	TObjectPtr<const UContextualAnimSceneAsset> SceneAsset = nullptr;

	// This property is deprecated and will be removed soon. AnimSetIdx is now stored per Binding
	UPROPERTY()
	int8 SectionIdx_DEPRECATED = INDEX_NONE;

	// This property is deprecated and will be removed soon. AnimSetIdx is now stored per Binding
	UPROPERTY()
	int8 AnimSetIdx_DEPRECATED = INDEX_NONE;

	/** List of actors bound to each role in the SceneAsset */
	UPROPERTY()
	TArray<FContextualAnimSceneBinding> Data;
	
	UPROPERTY()
	TScriptInterface<IContextualAnimOverrideInterface> OverrideProviderInterface = nullptr;
};

// DEPRECATED: Kept around only to do not break existing content. It will go away in the future.

USTRUCT(BlueprintType)
struct FContextualAnimStartSceneParams
{
	GENERATED_BODY()

	/** Map with actors to bind to each role in the scene */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TMap<FName, FContextualAnimSceneBindingContext> RoleToActorMap;

	/** Desired section. If INDEX_NONE the Manager will use or find best set in the first section. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	int32 SectionIdx = INDEX_NONE;

	/** Desired set. If INDEX_NONE the Manager will attempt to find the best set to use by running the selection criteria.
	 * The selection will be performed in the section specified by SectionIdx or in the first section if SectionIdx == INDEX_NONE.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	int32 AnimSetIdx = INDEX_NONE;

	/** Precomputed warp points that could be provided when starting a scene.
	 * When not provided, the warp points will be automatically computed using local context information.
	 */
	TArray<FContextualAnimWarpPoint> WarpPoints;

	void Reset()
	{
		RoleToActorMap.Reset();
		SectionIdx = INDEX_NONE;
		AnimSetIdx = INDEX_NONE;
		WarpPoints.Reset();
	}
};

///////////////////////////////////////////////////////////////////////

/** 
 * Stores the result of a query function 
 * @TODO: Only used by UContextualAnimSceneAsset::Query. Kept around only to do not break existing content. It will go away in the future.
 */
USTRUCT(BlueprintType)
struct FContextualAnimQueryResult
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	TWeakObjectPtr<UAnimMontage> Animation;

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	FTransform EntryTransform;

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	FTransform SyncTransform;

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	float AnimStartTime = 0.f;

	UPROPERTY(BlueprintReadWrite, Category = "Defaults")
	int32 AnimSetIdx = INDEX_NONE;

	void Reset()
	{
		Animation.Reset();
		EntryTransform = SyncTransform = FTransform::Identity;
		AnimStartTime = 0.f;
		AnimSetIdx = INDEX_NONE;
	}

	inline bool IsValid() const { return AnimSetIdx != INDEX_NONE; }
};

/** 
 * Stores the parameters passed into query function 
 * @TODO: Only used by UContextualAnimSceneAsset::Query. Kept around only to do not break existing content. It will go away in the future.
 */
USTRUCT(BlueprintType)
struct FContextualAnimQueryParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	TWeakObjectPtr<const AActor> Querier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	FTransform QueryTransform;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bComplexQuery = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Defaults")
	bool bFindAnimStartTime = false;

	FContextualAnimQueryParams() {}

	FContextualAnimQueryParams(const AActor* InQuerier, bool bInComplexQuery, bool bInFindAnimStartTime)
		: Querier(InQuerier), bComplexQuery(bInComplexQuery), bFindAnimStartTime(bInFindAnimStartTime) {}

	FContextualAnimQueryParams(const FTransform& InQueryTransform, bool bInComplexQuery, bool bInFindAnimStartTime)
		: QueryTransform(InQueryTransform), bComplexQuery(bInComplexQuery), bFindAnimStartTime(bInFindAnimStartTime) {}
};

#undef UE_API
