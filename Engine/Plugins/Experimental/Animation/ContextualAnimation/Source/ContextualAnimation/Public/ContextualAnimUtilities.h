// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContextualAnimTypes.h"
#include "ContextualAnimSceneAsset.h"

#include "ContextualAnimUtilities.generated.h"

#define UE_API CONTEXTUALANIMATION_API

class UContextualAnimSceneAsset;
class IContextualAnimOverrideInterface;
class UMeshComponent;
class USkeletalMeshComponent;
class UStaticMeshComponent;
class UAnimInstance;
class AActor;
class FPrimitiveDrawInterface;
struct FAnimMontageInstance;
struct FCompactPose;
struct FContextualAnimSet;
struct FAnimNotifyEvent;
template<class PoseType> struct FCSPose;

UCLASS(MinimalAPI)
class UContextualAnimUtilities : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** 
	 * Helper function to extract local space pose from an animation at a given time.
	 * If the supplied animation is a montage it will extract the pose from the first track
	 * IMPORTANT: This function expects you to add a MemMark (FMemMark Mark(FMemStack::Get());) at the correct scope if you are using it from outside world's tick
	 */
	static UE_API void ExtractLocalSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCompactPose& OutPose);

	/**
	 * Helper function to extract component space pose from an animation at a given time
     * If the supplied animation is a montage it will extract the pose from the first track
	 * IMPORTANT: This function expects you to add a MemMark (FMemMark Mark(FMemStack::Get());) at the correct scope if you are using it from outside world's tick
	 */
	static UE_API void ExtractComponentSpacePose(const UAnimSequenceBase* Animation, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose);

	/** Extract Root Motion transform from a contiguous position range */
	static UE_API FTransform ExtractRootMotionFromAnimation(const UAnimSequenceBase* Animation, float StartTime, float EndTime);

	/** Extract root bone transform at a given time */
	static UE_API FTransform ExtractRootTransformFromAnimation(const UAnimSequenceBase* Animation, float Time);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Utilities", meta = (WorldContext = "WorldContextObject", DisplayName = "Draw Debug Pose"))
	static UE_API void BP_DrawDebugPose(const UObject* WorldContextObject, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness);

	typedef TFunctionRef<void(const FVector& LineStart, const FVector& LineEnd, const FColor& Color, float LifeTime, float Thickness)> FDrawLineFunction;
	static UE_API void DrawPose(const UWorld* World, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness);
	static UE_API void DrawPose(FPrimitiveDrawInterface* PDI, const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float Thickness);
	static UE_API void DrawPose(const UAnimSequenceBase* Animation, float Time, FTransform LocalToWorldTransform, FLinearColor Color, float LifeTime, float Thickness, FDrawLineFunction DrawFunction);
	
	static UE_API void DrawDebugAnimSet(const UWorld* World, const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSet& AnimSet, float Time, const FTransform& ToWorldTransform, const FColor& Color, float LifeTime, float Thickness);

	static UE_API const FAnimNotifyEvent* FindFirstWarpingWindowForWarpTarget(const UAnimSequenceBase* Animation, FName WarpTargetName);

	static UE_API UMeshComponent* TryGetMeshComponentWithSocket(const AActor* Actor, FName SocketName);

	static UE_API USkeletalMeshComponent* TryGetSkeletalMeshComponent(const AActor* Actor);

	static UE_API UAnimInstance* TryGetAnimInstance(const AActor* Actor);

	static UE_API FAnimMontageInstance* TryGetActiveAnimMontageInstance(const AActor* Actor);

	static UE_API float GetSyncTimeForWarpSection(const UAnimSequenceBase* Animation, const FName& WarpSectionName);
	
	static UE_API void GetStartAndEndTimeForWarpSection(const UAnimSequenceBase* Animation, const FName& WarpSectionName, float& OutStartTime, float& OutEndTime);

	/** Returns the start/end time and warp target name of the Nth distinct warp section found sequentially in the animation's motion warping notifies */
	static UE_API FName GetStartAndEndTimeForWarpSection(const UAnimSequenceBase* Animation, int32 DesiredWarpSectionOrdinal, float& OutStartTime, float& OutEndTime);
	
	static UE_API void GenerateAlignmentTrack(const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FContextualAnimAlignmentTrackContainer& Container);
	static UE_API void GenerateIKTargetTrack(const UContextualAnimSceneAsset& SceneAsset, const FContextualAnimSceneBinding& InBinding, const FContextualAnimSceneBindings& InBindings, FContextualAnimAlignmentTrackContainer& Container);

	
	static UE_API void DrawSector(FPrimitiveDrawInterface& PDI, const FVector& Origin, const FVector& Direction, float MinDistance, float MaxDistance, float MinAngle, float MaxAngle, const FLinearColor& Color, uint8 DepthPriority, float Thickness, bool bDashedLine);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Create Contextual Anim Scene Bindings"))
	static UE_API bool BP_CreateContextualAnimSceneBindings(const UContextualAnimSceneAsset* SceneAsset, const TMap<FName, FContextualAnimSceneBindingContext>& Params, FContextualAnimSceneBindings& OutBindings);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Create Contextual Anim Scene Bindings For Two Actors"))
	static UE_API bool BP_CreateContextualAnimSceneBindingsForTwoActors(const UContextualAnimSceneAsset* SceneAsset, const FContextualAnimSceneBindingContext& Primary, const FContextualAnimSceneBindingContext& Secondary, FContextualAnimSceneBindings& OutBindings);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Asset", meta = (DisplayName = "Add Override Provider to Bindings"))
	static UE_API void BP_AddContextualAnimationOverrideProvider(UPARAM(Ref)FContextualAnimSceneBindings& InBindings, UPARAM(Ref)TScriptInterface<IContextualAnimOverrideInterface>& OverrideProvider, FContextualAnimSceneBindings& OutBindings);
	
	// Montage Blueprint Interface
	//------------------------------------------------------------------------------------------

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Utilities", meta = (DisplayName = "GetSectionStartAndEndTime"))
	static UE_API void BP_Montage_GetSectionStartAndEndTime(const UAnimMontage* Montage, int32 SectionIndex, float& OutStartTime, float& OutEndTime);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Utilities", meta = (DisplayName = "GetSectionTimeLeftFromPos"))
	static UE_API float BP_Montage_GetSectionTimeLeftFromPos(const UAnimMontage* Montage, float Position);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Utilities", meta = (DisplayName = "GetSectionLength"))
	static UE_API float BP_Montage_GetSectionLength(const UAnimMontage* Montage, int32 SectionIndex);

	// SceneBindings Blueprint Interface
	//------------------------------------------------------------------------------------------

	UE_DEPRECATED(5.7, "This function is deprecated. Please use the version that takes a Section Index")
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Calculate Warp Points For Bindings"))
	static UE_API void BP_SceneBindings_CalculateWarpPoints(const FContextualAnimSceneBindings& Bindings, TArray<FContextualAnimWarpPoint>& OutWarpPoints);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Calculate Warp Points For Bindings"))
	static UE_API void BP_SceneBindings_CalculateWarpPointsForSectionAtIndex(const FContextualAnimSceneBindings& Bindings, int32 SectionIdx, TArray<FContextualAnimWarpPoint>& OutWarpPoints);
	
	UE_DEPRECATED(5.7, "This function is deprecated. Please use the version that takes a Section Index")
	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Add Or Update Warp Targets For Bindings"))
	static UE_API void BP_SceneBindings_AddOrUpdateWarpTargetsForBindings(const FContextualAnimSceneBindings& Bindings);

	UFUNCTION(BlueprintCallable, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Add Or Update Warp Targets For Bindings"))
	static UE_API void BP_SceneBindings_AddOrUpdateWarpTargetsForSectionAtIndex(const FContextualAnimSceneBindings& Bindings, int32 SectionIdx);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Bindings"))
	static const TArray<FContextualAnimSceneBinding>& BP_SceneBindings_GetBindings(const FContextualAnimSceneBindings& Bindings) { return Bindings.GetBindings(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Binding By Role"))
	static UE_API const FContextualAnimSceneBinding& BP_SceneBindings_GetBindingByRole(const FContextualAnimSceneBindings& Bindings, FName Role);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Binding By Actor"))
	static UE_API const FContextualAnimSceneBinding& BP_SceneBindings_GetBindingByActor(const FContextualAnimSceneBindings& Bindings, const AActor* Actor);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Primary Binding"))
	static UE_API const FContextualAnimSceneBinding& BP_SceneBindings_GetPrimaryBinding(const FContextualAnimSceneBindings& Bindings);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Animation From Binding"))
	static UE_API const UAnimSequenceBase* BP_SceneBinding_GetAnimationFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Role From Binding"))
	static UE_API FName BP_SceneBinding_GetRoleFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Scene Asset"))
	static const UContextualAnimSceneAsset* BP_SceneBindings_GetSceneAsset(const FContextualAnimSceneBindings& Bindings) { return Bindings.GetSceneAsset(); }

	UE_DEPRECATED(5.7, "This function is deprecated. Please use BP_SceneBinding_GetSectionAndAnimSetIndices instead")
	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Section And AnimSet Indices"))
	static UE_API void BP_SceneBindings_GetSectionAndAnimSetIndices(const FContextualAnimSceneBindings& Bindings, int32& SectionIdx, int32& AnimSetIdx);

	UE_DEPRECATED(5.7, "This function is deprecated. Please use BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding instead")
	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Section And AnimSet Names"))
	static UE_API void BP_SceneBindings_GetSectionAndAnimSetNames(const FContextualAnimSceneBindings& Bindings, FName& SectionName, FName& AnimSetName);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Section And AnimSet Names From Binding"))
	static UE_API void BP_SceneBindings_GetSectionAndAnimSetNamesFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding, FName& SectionName, FName& AnimSetName);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Alignment Transform For Role Relative To Other Role"))
	static UE_API FTransform BP_SceneBindings_GetAlignmentTransformForRoleRelativeToOtherRole(const FContextualAnimSceneBindings& Bindings, FName Role, FName RelativeToRole, float Time);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Alignment Transform For Role Relative To Warp Point"))
	static UE_API FTransform BP_SceneBindings_GetAlignmentTransformForRoleRelativeToWarpPoint(const FContextualAnimSceneBindings& Bindings, FName Role, const FContextualAnimWarpPoint& WarpPoint, float Time);

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Bindings", meta = (DisplayName = "Get Alignment Transform From Binding"))
	static UE_API FTransform BP_SceneBindings_GetAlignmentTransformFromBinding(const FContextualAnimSceneBindings& Bindings, const FContextualAnimSceneBinding& Binding, const FContextualAnimWarpPoint& WarpPoint);

	// FContextualAnimSceneBindingContext Blueprint Interface
	//------------------------------------------------------------------------------------------
	
	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (NativeMakeFunc, DisplayName = "Make Contextual Anim Scene Binding Context"))
	static FContextualAnimSceneBindingContext BP_SceneBindingContext_MakeFromActor(AActor* Actor) { return FContextualAnimSceneBindingContext(Actor); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Make Contextual Anim Scene Binding Context With External Transform"))
	static FContextualAnimSceneBindingContext BP_SceneBindingContext_MakeFromActorWithExternalTransform(AActor* Actor, FTransform ExternalTransform) { return FContextualAnimSceneBindingContext(Actor, ExternalTransform); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Get Actor"))
	static AActor* BP_SceneBindingContext_GetActor(const FContextualAnimSceneBindingContext& BindingContext) { return BindingContext.GetActor(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Get Transform"))
	static FTransform BP_SceneBindingContext_GetTransform(const FContextualAnimSceneBindingContext& BindingContext) { return BindingContext.GetTransform(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Get Velocity"))
	static FVector BP_SceneBindingContext_GetVelocity(const FContextualAnimSceneBindingContext& BindingContext) { return BindingContext.GetVelocity(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Get GameplayTags"))
	static const FGameplayTagContainer& BP_SceneBindingContext_GetGameplayTags(const FContextualAnimSceneBindingContext& BindingContext) { return BindingContext.GetGameplayTags(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Has Matching GameplayTag"))
	static bool BP_SceneBindingContext_HasMatchingGameplayTag(const FContextualAnimSceneBindingContext& BindingContext, const FGameplayTag& TagToCheck) { return BindingContext.HasMatchingGameplayTag(TagToCheck); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Has All Matching GameplayTags"))
	static bool BP_SceneBindingContext_HasAllMatchingGameplayTags(const FContextualAnimSceneBindingContext& BindingContext, const FGameplayTagContainer& TagContainer) { return BindingContext.HasAllMatchingGameplayTags(TagContainer); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Has Any Matching GameplayTags"))
	static bool BP_SceneBindingContext_HasAnyMatchingGameplayTags(const FContextualAnimSceneBindingContext& BindingContext, const FGameplayTagContainer& TagContainer) { return BindingContext.HasAnyMatchingGameplayTags(TagContainer); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding Context", meta = (DisplayName = "Get Current Section and Anim Set"))
	static UE_API void BP_SceneBindingContext_GetCurrentSectionAndAnimSetNames(const FContextualAnimSceneBindingContext& BindingContext, FName& SectionName, FName& AnimSetName);


	// FContextualAnimSceneBinding Blueprint Interface
	//------------------------------------------------------------------------------------------

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding", meta = (DisplayName = "Get Actor"))
	static AActor* BP_SceneBinding_GetActor(const FContextualAnimSceneBinding& Binding) { return Binding.GetActor(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding", meta = (DisplayName = "Get Skeletal Mesh"))
	static USkeletalMeshComponent* BP_SceneBinding_GetSkeletalMesh(const FContextualAnimSceneBinding& Binding) { return Binding.GetSkeletalMeshComponent(); }

	UFUNCTION(BlueprintPure, Category = "Contextual Anim|Scene Binding", meta = (DisplayName = "Get Section And AnimSet Indices"))
	static UE_API void BP_SceneBinding_GetSectionAndAnimSetIndices(const FContextualAnimSceneBinding& Binding, int32& SectionIdx, int32& AnimSetIdx);

	static void ExtractPoseIgnoringForceRootLock(UAnimSequenceBase* AnimSequenceBase, const FBoneContainer& BoneContainer, float Time, bool bExtractRootMotion, FCSPose<FCompactPose>& OutPose);
	static FCompactPoseBoneIndex GetCompactPoseBoneIndexFromPose(const FCSPose<FCompactPose>& Pose, const FName& BoneName);
	
	static UE_API void GenerateAlignmentTracksForBindings(UContextualAnimSceneAsset& SceneAsset, FContextualAnimSceneBindings& InBindings);
	static UE_API void GenerateAlignmentTrackForBinding(UContextualAnimSceneAsset& SceneAsset, const FName& Role, FContextualAnimSceneBinding& Binding);
};

#undef UE_API
