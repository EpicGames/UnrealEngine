// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ContextualAnimationEditor : ModuleRules
{
	public ContextualAnimationEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AIModule",
				"NavigationSystem",
				"ContextualAnimation",
				"MotionWarping",
				"GameplayTags"
			}
			);			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"LevelEditor",
				"DetailCustomizations",
				"EditorStyle",
				"Projects",
				"AssetTools",
				"EditorWidgets",
				"PropertyEditor",
				"KismetWidgets",
				"AdvancedPreviewScene",
				"GraphEditor",
				"SequencerWidgets",
				"MovieScene",
				"SequencerCore",
				"Sequencer",
				"MovieSceneTracks",
				"MovieSceneTools",
				"ClassViewer",
				"Persona",
				"RenderCore",
				"BlueprintGraph",
				"AnimGraph"
			}
			);
	}
}
