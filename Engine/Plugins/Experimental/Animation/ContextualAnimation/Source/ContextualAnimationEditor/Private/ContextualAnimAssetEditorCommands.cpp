// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimAssetEditorCommands.h"

#define LOCTEXT_NAMESPACE "ContextualAnimAssetEditorCommands"

void FContextualAnimAssetEditorCommands::RegisterCommands()
{
	UI_COMMAND(ResetPreviewScene, "Reset Scene", "Reset Scene.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NewAnimSet, "New AnimSet", "New AnimSet.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ShowIKTargetsDrawSelected, "Selected Actor Only", "Show IK Targets for the selected actor", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowIKTargetsDrawAll, "All Actors", "Show IK Targets for all the actors", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowIKTargetsDrawNone, "None", "Hide IK Targets", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ShowSelectionCriteriaActiveSet, "Active AnimSet", "Show selection criteria only for the active AnimSet", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowSelectionCriteriaAllSets, "All AnimSets", "Show selection criteria for all the AnimSets", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowSelectionCriteriaNone, "None", "Hide selection criteria", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ShowEntryPosesActiveSet, "Active AnimSet", "Show entry poses only for the active AnimSet", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowEntryPosesAllSets, "All AnimSets", "Show entry poses for all the AnimSets", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ShowEntryPosesNone, "None", "Hide Entry Poses", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(Simulate, "Simulate", "Simulate Mode", EUserInterfaceActionType::RadioButton, FInputChord());

}

#undef LOCTEXT_NAMESPACE
