// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimAssetEditorToolkit.h"
#include "Framework/Commands/UICommandList.h"
#include "SContextualAnimViewport.h"
#include "IDetailsView.h"
#include "SContextualAnimAssetBrowser.h"
#include "SContextualAnimNewAnimSetDialog.h"
#include "ContextualAnimPreviewScene.h"
#include "ContextualAnimAssetEditorCommands.h"
#include "ContextualAnimEditorTypes.h"
#include "Widgets/Docking/SDockTab.h"
#include "ContextualAnimViewModel.h"
#include "ContextualAnimSceneAsset.h"
#include "ISequencer.h"
#include "AdvancedPreviewSceneModule.h"
#include "Modules/ModuleManager.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "ContextualAnimAssetEditorToolkit"

const FName ContextualAnimEditorAppName = FName(TEXT("ContextualAnimEditorApp"));

// Tab identifiers
struct FContextualAnimEditorTabs
{
	static const FName AssetDetailsID;
	static const FName ViewportID;
	static const FName TimelineID;
	static const FName AssetBrowserID;
	static const FName PreviewSettingsID;
};

const FName FContextualAnimEditorTabs::AssetDetailsID(TEXT("ContextualAnimEditorAssetDetailsTabID"));
const FName FContextualAnimEditorTabs::ViewportID(TEXT("ContextualAnimEditorViewportTabID"));
const FName FContextualAnimEditorTabs::TimelineID(TEXT("ContextualAnimEditorTimelineTabID"));
const FName FContextualAnimEditorTabs::AssetBrowserID(TEXT("ContextualAnimEditorAssetBrowserTabID"));
const FName FContextualAnimEditorTabs::PreviewSettingsID(TEXT("ContextualAnimEditorPreviewSettingsTabID"));

FContextualAnimAssetEditorToolkit::FContextualAnimAssetEditorToolkit()
{
}

FContextualAnimAssetEditorToolkit::~FContextualAnimAssetEditorToolkit()
{
	check(ViewModel.IsValid());
	ViewModel->Shutdown();
}

UContextualAnimSceneAsset* FContextualAnimAssetEditorToolkit::GetSceneAsset() const
{
	return ViewModel.IsValid() ? ViewModel->GetSceneAsset() : nullptr;
}

void FContextualAnimAssetEditorToolkit::ResetPreviewScene()
{
	ViewModel->SetDefaultMode();
}

void FContextualAnimAssetEditorToolkit::InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UContextualAnimSceneAsset* SceneAsset)
{
	// Bind Commands
	BindCommands();

	// Create Preview Scene
	if (!PreviewScene.IsValid())
	{
		PreviewScene = MakeShareable(new FContextualAnimPreviewScene(FPreviewScene::ConstructionValues().AllowAudioPlayback(true).ShouldSimulatePhysics(false).ForceUseMovementComponentInNonGameWorld(true),
			StaticCastSharedRef<FContextualAnimAssetEditorToolkit>(AsShared())));
	}

	// Create viewport widget
	FContextualAnimViewportRequiredArgs ViewportArgs(StaticCastSharedRef<FContextualAnimAssetEditorToolkit>(AsShared()), PreviewScene.ToSharedRef());
	ViewportWidget = SNew(SContextualAnimViewport, ViewportArgs);

	// Create asset browser widget
	AssetBrowserWidget = SNew(SContextualAnimAssetBrowser);

	ViewModel = MakeShared<FContextualAnimViewModel>();
	ViewModel->Initialize(SceneAsset, PreviewScene.ToSharedRef());

	// Define Editor Layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_ContextualAnimAnimEditor_Layout_v0.13")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.65f)
							->AddTab(FContextualAnimEditorTabs::ViewportID, ETabState::OpenedTab)->SetHideTabWell(true)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.3f)
							->AddTab(FContextualAnimEditorTabs::TimelineID, ETabState::OpenedTab)
						)
					)
					->Split
					(
						FTabManager::NewSplitter()->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.3f)
							->AddTab(FContextualAnimEditorTabs::AssetDetailsID, ETabState::OpenedTab)
							->AddTab(FContextualAnimEditorTabs::PreviewSettingsID, ETabState::OpenedTab)
						)
					)
				)
			)
		);

	const bool bCreateDefaultStandaloneMenuParam = true;
	const bool bCreateDefaultToolbarParam = true;
	const bool bIsToolbarFocusableParam = false;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, ContextualAnimEditorAppName, StandaloneDefaultLayout, bCreateDefaultStandaloneMenuParam, bCreateDefaultToolbarParam, SceneAsset, bIsToolbarFocusableParam);

	ExtendToolbar();

	RegenerateMenusAndToolbars();
}

void FContextualAnimAssetEditorToolkit::BindCommands()
{
	const FContextualAnimAssetEditorCommands& Commands = FContextualAnimAssetEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.ResetPreviewScene,
		FExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::ResetPreviewScene),
		FCanExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::CanMakeEdits),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.NewAnimSet,
		FExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::ShowNewAnimSetDialog),
		FCanExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::CanMakeEdits),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.Simulate,
		FExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::ToggleSimulateMode),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FContextualAnimAssetEditorToolkit::IsSimulateModeActive));
}

void FContextualAnimAssetEditorToolkit::ToggleSimulateMode()
{
	if(ViewModel.IsValid())
	{
		ViewModel->ToggleSimulateMode();
	}
}

bool FContextualAnimAssetEditorToolkit::IsSimulateModeActive() const
{
	return (ViewModel.IsValid() && !ViewModel->IsSimulateModeInactive());
}

bool FContextualAnimAssetEditorToolkit::CanMakeEdits() const
{
	return (ViewModel.IsValid() && ViewModel->CanMakeEdits());
}

void FContextualAnimAssetEditorToolkit::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateSP(this, &FContextualAnimAssetEditorToolkit::FillToolbar)
	);
}

void FContextualAnimAssetEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.AddToolBarButton(
		FContextualAnimAssetEditorCommands::Get().ResetPreviewScene,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh")
	);

	ToolbarBuilder.AddComboButton(
		FUIAction(
			FExecuteAction(),
			FCanExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::CanMakeEdits),
			FIsActionChecked()
		),
		FOnGetContent::CreateSP(this, &FContextualAnimAssetEditorToolkit::BuildSectionsMenu),
		LOCTEXT("Sections_Label", "Sections"),
		FText::GetEmpty(),
		FSlateIcon()
	);
	
	ToolbarBuilder.AddComboButton(
	FUIAction(
		FExecuteAction(),
		FCanExecuteAction::CreateSP(this, &FContextualAnimAssetEditorToolkit::CanMakeEdits),
		FIsActionChecked()
	),
		FOnGetContent::CreateSP(this, &FContextualAnimAssetEditorToolkit::BuildAnimSetMenu),
		LOCTEXT("AnimSet_Label", "New AnimSet"),
		FText::GetEmpty(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
	);

	ToolbarBuilder.AddToolBarButton(
		FContextualAnimAssetEditorCommands::Get().Simulate,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon()
	);
}

TSharedRef<SWidget> FContextualAnimAssetEditorToolkit::BuildSectionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("Sections_Label", "Sections"));
	{
		const UContextualAnimSceneAsset* SceneAsset = GetSceneAsset();
		
		TArray<FName> SectionNames = SceneAsset->GetSectionNames();
		for (int32 Idx = 0; Idx < SectionNames.Num(); Idx++)
		{
			MenuBuilder.AddMenuEntry(
				FText::FromString(SectionNames[Idx].ToString()),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([this, Idx]() {
					ViewModel->SetActiveSection(Idx);
				})));
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FContextualAnimAssetEditorToolkit::BuildAnimSetMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AnimSet_Label", "New AnimSet"));
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString("Add New AnimSet"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() {
				ShowNewAnimSetDialog();
			})));
		MenuBuilder.AddMenuEntry(
			FText::FromString("Add Empty AnimSet to Active Section"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this]() {AddAnimSetToActiveSection();
			})));
	}

	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

void FContextualAnimAssetEditorToolkit::ShowNewAnimSetDialog()
{
	const UContextualAnimSceneAsset* SceneAsset = GetSceneAsset();
	const int32 NumRoles = SceneAsset ? SceneAsset->GetNumRoles() : 0;

	if (NumRoles == 0)
	{
		const FText DialogMsg = LOCTEXT("MissingRolesDialog", "Can't add new anim set. Roles asset hasn't been set or no roles has been defined.");
		FMessageDialog::Open(EAppMsgType::Ok, DialogMsg);
		return;
	}

	TSharedRef<SContextualAnimNewAnimSetDialog> NewAnimSetDialog = SNew(SContextualAnimNewAnimSetDialog, ViewModel.ToSharedRef());
	NewAnimSetDialog->Show();
}

void FContextualAnimAssetEditorToolkit::AddAnimSetToActiveSection()
{
	const UContextualAnimSceneAsset* SceneAsset = GetSceneAsset();
	const int32 NumRoles = SceneAsset ? SceneAsset->GetNumRoles() : 0;
	

	if (NumRoles == 0)
	{
		const FText DialogMsg = LOCTEXT("MissingRolesDialog", "Can't add new anim set. Roles asset hasn't been set or no roles has been defined.");
		FMessageDialog::Open(EAppMsgType::Ok, DialogMsg);
		return;
	}
	
	FContextualAnimNewAnimSetParams Params = FContextualAnimNewAnimSetParams();
	const TArray<FName> Roles = SceneAsset->GetRoles();
	
	for (FName RoleName: Roles)
	{
		FContextualAnimNewAnimSetData RoleData;
		RoleData.RoleName = RoleName; 
		Params.Data.Add(RoleData);
	}
	
	const int32 ActiveSectionIdx = ViewModel->GetActiveSection();
	Params.SectionName = SceneAsset->GetSection(ActiveSectionIdx)->GetName();
	ViewModel->AddNewAnimSet(Params);
}

void FContextualAnimAssetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_CASEditor", "Contextual Animation Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(FContextualAnimEditorTabs::ViewportID, FOnSpawnTab::CreateSP(this, &FContextualAnimAssetEditorToolkit::SpawnTab_Viewport))
		.SetDisplayName(LOCTEXT("ViewportTab", "Viewport"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "GraphEditor.EventGraph_16x"));

	InTabManager->RegisterTabSpawner(FContextualAnimEditorTabs::AssetDetailsID, FOnSpawnTab::CreateSP(this, &FContextualAnimAssetEditorToolkit::SpawnTab_AssetDetails))
		.SetDisplayName(LOCTEXT("AssetDetailsTab", "AssetDetails"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(FContextualAnimEditorTabs::TimelineID, FOnSpawnTab::CreateSP(this, &FContextualAnimAssetEditorToolkit::SpawnTab_Timeline))
		.SetDisplayName(LOCTEXT("TimelineTab", "Timeline"))
		.SetGroup(WorkspaceMenuCategoryRef);

 	InTabManager->RegisterTabSpawner(FContextualAnimEditorTabs::AssetBrowserID, FOnSpawnTab::CreateSP(this, &FContextualAnimAssetEditorToolkit::SpawnTab_AssetBrowser))
 		.SetDisplayName(LOCTEXT("AssetBrowserTab", "AssetBrowser"))
 		.SetGroup(WorkspaceMenuCategoryRef);

	InTabManager->RegisterTabSpawner(FContextualAnimEditorTabs::PreviewSettingsID, FOnSpawnTab::CreateSP(this, &FContextualAnimAssetEditorToolkit::SpawnTab_PreviewSettings))
		.SetDisplayName(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		.SetGroup(WorkspaceMenuCategoryRef)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));
}

void FContextualAnimAssetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);

	InTabManager->UnregisterTabSpawner(FContextualAnimEditorTabs::ViewportID);
	InTabManager->UnregisterTabSpawner(FContextualAnimEditorTabs::AssetDetailsID);
	InTabManager->UnregisterTabSpawner(FContextualAnimEditorTabs::TimelineID);
	InTabManager->UnregisterTabSpawner(FContextualAnimEditorTabs::AssetBrowserID);
	InTabManager->UnregisterTabSpawner(FContextualAnimEditorTabs::PreviewSettingsID);
}

FName FContextualAnimAssetEditorToolkit::GetToolkitFName() const
{
	return FName("ContextualAnimEditor");
}

FText FContextualAnimAssetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ContextualAnimEditorAppLabel", "Contextual Anim Editor");
}

FText FContextualAnimAssetEditorToolkit::GetToolkitName() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("AssetName"), FText::FromString(GetSceneAsset()->GetName()));
	return FText::Format(LOCTEXT("ContextualAnimEditorToolkitName", "{AssetName}"), Args);
}

FLinearColor FContextualAnimAssetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FContextualAnimAssetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return TEXT("ContextualAnimEditor");
}

TSharedRef<SDockTab> FContextualAnimAssetEditorToolkit::SpawnTab_Viewport(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FContextualAnimEditorTabs::ViewportID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(LOCTEXT("ViewportTab_Title", "Viewport"));

	if (ViewportWidget.IsValid())
	{
		SpawnedTab->SetContent(ViewportWidget.ToSharedRef());
	}

	return SpawnedTab;
}

TSharedRef<SDockTab> FContextualAnimAssetEditorToolkit::SpawnTab_Timeline(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId().TabType == FContextualAnimEditorTabs::TimelineID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(LOCTEXT("TimelineTab_Title", "Timeline"));

	TSharedPtr<ISequencer> SequencerPtr = ViewModel ? ViewModel->GetSequencer() : nullptr;
	if (ViewModel && ViewModel->GetSequencer() != nullptr)
	{
		SpawnedTab->SetContent(SequencerPtr->GetSequencerWidget());
	}

	return SpawnedTab;
}

TSharedRef<SDockTab> FContextualAnimAssetEditorToolkit::SpawnTab_AssetDetails(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FContextualAnimEditorTabs::AssetDetailsID);

	return SNew(SDockTab)
		.Label(LOCTEXT("AssetDetails_Title", "Asset Details"))
		[
			ViewModel->GetDetailsView().ToSharedRef()
		];
}

TSharedRef<SDockTab> FContextualAnimAssetEditorToolkit::SpawnTab_AssetBrowser(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FContextualAnimEditorTabs::AssetBrowserID);

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab).Label(LOCTEXT("EditorAssetBrowser_Title", "Asset Browser"));

	if (AssetBrowserWidget.IsValid())
	{
		SpawnedTab->SetContent(AssetBrowserWidget.ToSharedRef());
	}

	return SpawnedTab;
}

TSharedRef<SDockTab> FContextualAnimAssetEditorToolkit::SpawnTab_PreviewSettings(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FContextualAnimEditorTabs::PreviewSettingsID);

	FAdvancedPreviewSceneModule& AdvancedPreviewSceneModule = FModuleManager::LoadModuleChecked<FAdvancedPreviewSceneModule>("AdvancedPreviewScene");
	TSharedRef<SWidget> InWidget= AdvancedPreviewSceneModule.CreateAdvancedPreviewSceneSettingsWidget(PreviewScene.ToSharedRef());

	TSharedRef<SDockTab> SpawnedTab = SNew(SDockTab)
		.Label(LOCTEXT("PreviewSceneSettingsTab", "Preview Scene Settings"))
		[
			InWidget
		];

	return SpawnedTab;
}

#undef LOCTEXT_NAMESPACE

