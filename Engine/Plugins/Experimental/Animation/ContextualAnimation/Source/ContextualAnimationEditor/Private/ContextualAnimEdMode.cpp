// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimEdMode.h"
#include "EngineUtils.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimSelectionCriterion.h"
#include "ContextualAnimUtilities.h"
#include "ContextualAnimViewportClient.h"
#include "ContextualAnimAssetEditorToolkit.h"
#include "ContextualAnimViewModel.h"
#include "AnimNotifyState_IKWindow.h"
#include "SceneView.h"
#include "SkeletalDebugRendering.h"
#include "CanvasTypes.h"
#include "CanvasItem.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/CapsuleComponent.h"
#include "GameFramework/Character.h"

const FEditorModeID FContextualAnimEdMode::EdModeId = TEXT("ContextualAnimEdMode");

IMPLEMENT_HIT_PROXY(HSelectionCriterionHitProxy, HHitProxy);

FContextualAnimEdMode::FContextualAnimEdMode()
{
}

FContextualAnimEdMode::~FContextualAnimEdMode()
{
}

void FContextualAnimEdMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);

	if (ViewModel)
	{
		ViewModel->TickSimulateMode(DeltaTime);
	}
}

void FContextualAnimEdMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	// @TODO: This should not be initialized here
	FContextualAnimViewportClient* ViewportClient = static_cast<FContextualAnimViewportClient*>(Viewport->GetClient());
	if (!ViewModel)
	{
		ViewModel = ViewportClient->GetAssetEditorToolkit()->GetViewModel();
	}

	if (ViewModel)
	{
		if (const UContextualAnimSceneAsset* SceneAsset = ViewModel->GetSceneAsset())
		{
			const FContextualAnimSceneBindings& Bindings = ViewModel->SceneBindings;

			// If any bound actor became invalid (e.g., Blueprint recompilation destroying the old instance),
			// refresh the preview scene to re-spawn actors from the current classes.
			for (const FContextualAnimSceneBinding& Binding : Bindings)
			{
				if (!Binding.GetActor())
				{
					ViewModel->SetDefaultMode();
					return;
				}
			}

			const FName PrimaryRole = SceneAsset->GetPrimaryRole();
			const FContextualAnimSceneBinding* PrimaryBinding = Bindings.FindBindingByRole(PrimaryRole);

			if(PrimaryBinding == nullptr)
			{
				return;
			}

			const FTransform PrimaryTransform = PrimaryBinding->GetContext().GetTransform();
			if (const FContextualAnimSceneSection* Section = SceneAsset->GetSection(PrimaryBinding->GetSectionIdx()))
			{
				// Draw warp points
				if (const FContextualAnimSet* AnimSet = Section->GetAnimSet(PrimaryBinding->GetAnimSetIdx()))
				{
					for (const FContextualAnimWarpPointDefinition& WarpPointDef : Section->GetWarpPointDefinitions())
					{
						FContextualAnimWarpPoint WarpPoint;
						if (Bindings.CalculateWarpPoint(WarpPointDef, WarpPoint))
						{
							// Draw warp point in Green.
							const float Thickness = 0.f;
							DrawCoordinateSystem(PDI, WarpPoint.Transform.GetLocation(), WarpPoint.Transform.Rotator(), 5.f, SDPG_Foreground, Thickness);
							DrawWireDiamond(PDI, WarpPoint.Transform.ToMatrixNoScale(), 3.f, FLinearColor::Green, 0, Thickness);

							// Draw warp target transforms for each actor using this warp point in White. This is where the root should end up at the end of the warping window.
							for (const FContextualAnimSceneBinding& Binding : Bindings)
							{
								const UAnimSequenceBase* AnimationSequence = Binding.GetAnimation();
								if(AnimationSequence)
								{
									const float Time = UContextualAnimUtilities::GetSyncTimeForWarpSection(AnimationSequence, WarpPointDef.WarpTargetName);
									const FTransform TransformRelativeToWarpPoint = SceneAsset->GetAlignmentTransform(Binding.GetSectionIdx(), Binding.GetAnimSetIdx(), Binding.GetAnimTrackIdx(), WarpPointDef.WarpTargetName, Time);
									const FTransform WarpTargetTransform = (TransformRelativeToWarpPoint * WarpPoint.Transform);

									DrawCoordinateSystem(PDI, WarpTargetTransform.GetLocation(), WarpTargetTransform.Rotator(), 5.f, SDPG_Foreground, Thickness);
									DrawWireDiamond(PDI, WarpTargetTransform.ToMatrixNoScale(), 3.f, FLinearColor::White, 0, Thickness);
								}
							}
						}
					}
				}

				// Attempt to draw selection criteria and entry poses only when playback time is at 0 and simulate mode is not playing
				const float PlaybackTime = ViewModel->GetPlaybackTime();
				if (PlaybackTime <= 0.f && !ViewModel->IsSimulateModePlaying())
				{
					for (int32 AnimSetIdx = 0; AnimSetIdx < Section->GetNumAnimSets(); AnimSetIdx++)
					{
						const bool bIsActiveSet = AnimSetIdx == PrimaryBinding->GetAnimSetIdx();

						for (const FContextualAnimTrack& AnimTrack : Section->GetAnimSet(AnimSetIdx)->Tracks)
						{
							const FContextualAnimSceneBinding* Binding = Bindings.FindBindingByRole(AnimTrack.Role);

							if (Binding == nullptr)
							{
								continue;
							}

							bool bPassesSelectionCriteria = true;
							for (int32 CriterionIdx = 0; CriterionIdx < AnimTrack.SelectionCriteria.Num(); CriterionIdx++)
							{
								if (const UContextualAnimSelectionCriterion* Criterion = AnimTrack.SelectionCriteria[CriterionIdx])
								{
									const bool bPassesCurrentSelectionCriterion = Criterion->DoesQuerierPassCondition(PrimaryBinding->GetContext(), Binding->GetContext());
									if (!bPassesCurrentSelectionCriterion && bPassesSelectionCriteria)
									{
										bPassesSelectionCriteria = false;
									}

									// Draw Selection Criteria
									const EMultiOptionDrawMode SelectionCriteriaDrawMode = ViewportClient->GetSelectionCriteriaDrawMode();
									if (SelectionCriteriaDrawMode == EMultiOptionDrawMode::All || (SelectionCriteriaDrawMode == EMultiOptionDrawMode::Single && bIsActiveSet))
									{
										// Draw with dashed lines if this is not the active AnimSet and simulate mode is inactive
										const bool bUseDashedLines = !bIsActiveSet && ViewModel->IsSimulateModeInactive();

										FLinearColor DrawColor = FLinearColor::White;
										float Thickness = 1.f;
										if (bPassesCurrentSelectionCriterion)
										{
											DrawColor = FLinearColor::Green;
											Thickness = 2.f;
										}

										//@TODO: Each SelectionCriterion should implement this, and here we should just call "Criterion->Draw()"
										if (const UContextualAnimSelectionCriterion_TriggerArea* Spatial = Cast<UContextualAnimSelectionCriterion_TriggerArea>(Criterion))
										{
											const int32 LastIndex = Spatial->PolygonPoints.Num() - 1;
											for (int32 Idx = 0; Idx <= LastIndex; Idx++)
											{
												const FVector P0 = PrimaryTransform.TransformPositionNoScale(Spatial->PolygonPoints[Idx]);
												const FVector P1 = PrimaryTransform.TransformPositionNoScale(Spatial->PolygonPoints[Idx == LastIndex ? 0 : Idx + 1]);

												if (bUseDashedLines)
												{
													DrawDashedLine(PDI, P0, P1, DrawColor, 10.f, SDPG_Foreground);
													DrawDashedLine(PDI, P0 + FVector::UpVector * Spatial->Height, P1 + FVector::UpVector * Spatial->Height, DrawColor, 10.f, SDPG_Foreground);
													DrawDashedLine(PDI, P0, P0 + FVector::UpVector * Spatial->Height, DrawColor, 10.f, SDPG_Foreground);
												}
												else
												{
													PDI->DrawLine(P0, P1, DrawColor, SDPG_Foreground, Thickness);
													PDI->DrawLine(P0 + FVector::UpVector * Spatial->Height, P1 + FVector::UpVector * Spatial->Height, DrawColor, SDPG_Foreground, Thickness);
													PDI->DrawLine(P0, P0 + FVector::UpVector * Spatial->Height, DrawColor, SDPG_Foreground, Thickness);
												}

												// Draw hit proxy points only for the active set and if simulate mode is not active
												if (ViewModel->IsSimulateModeInactive() && bIsActiveSet)
												{
													PDI->SetHitProxy(new HSelectionCriterionHitProxy(AnimTrack.Role, CriterionIdx, Idx));
													PDI->DrawPoint(P0, FLinearColor::Black, 15.f, SDPG_Foreground);
													PDI->SetHitProxy(nullptr);

													PDI->SetHitProxy(new HSelectionCriterionHitProxy(AnimTrack.Role, CriterionIdx, Idx + 4));
													PDI->DrawPoint(P0 + FVector::UpVector * Spatial->Height, FLinearColor::Black, 15.f, SDPG_Foreground);
													PDI->SetHitProxy(nullptr);
												}
											}
										}
										else if (const UContextualAnimSelectionCriterion_Cone* Cone = Cast<UContextualAnimSelectionCriterion_Cone>(Criterion))
										{
											FVector Origin, Direction;
											if (Cone->Mode == EContextualAnimCriterionConeMode::ToPrimary)
											{
												const FTransform QuerierTransform = SceneAsset->GetAlignmentTransformForRoleRelativeToOtherRole(AnimTrack.SectionIdx, AnimTrack.AnimSetIdx, AnimTrack.Role, PrimaryRole, 0.f) * PrimaryTransform;
												Origin = QuerierTransform.GetLocation();
												Direction = QuerierTransform.GetRotation().GetForwardVector().RotateAngleAxis(Cone->Offset, FVector::UpVector);
											}
											else if (Cone->Mode == EContextualAnimCriterionConeMode::FromPrimary)
											{
												Origin = PrimaryTransform.GetLocation();
												Direction = PrimaryTransform.GetRotation().GetForwardVector().RotateAngleAxis(Cone->Offset, FVector::UpVector);
											}

											UContextualAnimUtilities::DrawSector(*PDI, Origin, Direction, 0.f, Cone->Distance, -Cone->HalfAngle, Cone->HalfAngle, DrawColor, SDPG_World, Thickness, bUseDashedLines);
										}
									}
								}
							}

							// Draw Entry Poses
							const EMultiOptionDrawMode EntryPosesDrawMode = ViewportClient->GetEntryPosesDrawMode();
							if (EntryPosesDrawMode == EMultiOptionDrawMode::All || (EntryPosesDrawMode == EMultiOptionDrawMode::Single && bIsActiveSet))
							{
								const UAnimSequenceBase* TrackAnimationSequence = Binding->GetAnimation();
								if (AnimTrack.Role != PrimaryRole && TrackAnimationSequence)
								{
									const float Time = 0.f;
									const FTransform RelativeToPrimary = SceneAsset->GetAlignmentTransformForRoleRelativeToOtherRole(AnimTrack.SectionIdx, AnimTrack.AnimSetIdx, AnimTrack.Role, PrimaryRole, Time);
									FTransform Transform = (FTransform(SceneAsset->GetMeshToComponentForRole(AnimTrack.Role).GetRotation()) * RelativeToPrimary) * PrimaryTransform;

									if (ACharacter* PrimaryAsCharacter = Cast<ACharacter>(PrimaryBinding->GetActor()))
									{
										const float CapsuleHalfHeight = PrimaryAsCharacter->GetCapsuleComponent()->GetScaledCapsuleHalfHeight();
										Transform.SetLocation(Transform.GetLocation() - (FVector::UpVector * CapsuleHalfHeight));
									}

									UContextualAnimUtilities::DrawPose(PDI, TrackAnimationSequence, Time, Transform, bPassesSelectionCriteria ? FColor::Green : FColor::White, 1.f);
								}
							}
						}
					}
				}

				// Draw IK Targets
				const EMultiOptionDrawMode IKTargetsDrawMode = ViewportClient->GetIKTargetsDrawMode();
				if (IKTargetsDrawMode != EMultiOptionDrawMode::None)
				{
					for (const FContextualAnimSceneBinding& Binding : Bindings)
					{
						if (IKTargetsDrawMode == EMultiOptionDrawMode::All || (IKTargetsDrawMode == EMultiOptionDrawMode::Single && Binding.GetActor() == ViewModel->GetSelectedActor()))
						{
							DrawIKTargetsForBinding(*PDI, Binding);
						}
					}
				}
			}
		}
	}
}

void FContextualAnimEdMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);

	if (ViewModel)
	{
		const FText DebugText = ViewModel->GetSelectionDebugText();
		const UFont* Font = GEngine->GetSmallFont();
		const FVector2D TextPos(10.f, 10.f);
		const float Padding = 6.f;

		// Measure text size for the background
		int32 TextWidth = 0;
		int32 TextHeight = 0;
		StringSize(Font, TextWidth, TextHeight, *DebugText.ToString());

		// Draw semi-transparent black background
		const FVector2D BackgroundPos(TextPos.X - Padding, TextPos.Y - Padding);
		const FVector2D BackgroundSize(TextWidth + Padding * 2.f, TextHeight + Padding * 2.f);
		FCanvasTileItem BackgroundTile(BackgroundPos, BackgroundSize, FLinearColor(0.f, 0.f, 0.f, 0.5f));
		BackgroundTile.BlendMode = SE_BLEND_Translucent;
		Canvas->DrawItem(BackgroundTile);

		// Draw text
		FCanvasTextItem TextItem(TextPos, DebugText, Font, FLinearColor::White);
		Canvas->DrawItem(TextItem);
	}
}

void FContextualAnimEdMode::DrawIKTargetsForBinding(FPrimitiveDrawInterface& PDI, const FContextualAnimSceneBinding& Binding) const
{
	const USkeletalMeshComponent* BindingSkelMeshComp = Binding.GetSkeletalMeshComponent();
	if (!BindingSkelMeshComp)
	{
		return;
	}

	const TArray<FContextualAnimIKTargetDefinition>& IKTargetDefs = ViewModel->SceneBindings.GetIKTargetDefContainerFromBinding(Binding).IKTargetDefs;

	for (const FContextualAnimIKTargetDefinition& IKTargetDef : IKTargetDefs)
	{
		if (const FContextualAnimSceneBinding* TargetBinding = ViewModel->SceneBindings.FindBindingByRole(IKTargetDef.TargetRoleName))
		{
			if (UMeshComponent* TargetMeshComp = UContextualAnimUtilities::TryGetMeshComponentWithSocket(TargetBinding->GetActor(), IKTargetDef.TargetBoneName))
			{
				const float Radius = 1.f;
				const float Alpha = UAnimNotifyState_IKWindow::GetIKAlphaValue(IKTargetDef.GoalName, Binding.GetAnimMontageInstance());
				FLinearColor Color = Alpha > 0.f ? FLinearColor(FColor::MakeRedToGreenColorFromScalar(Alpha)) : FLinearColor::White;

				if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Bone)
				{
					const FTransform SourceTransform = BindingSkelMeshComp->GetBoneTransform(IKTargetDef.BoneName);
					const FTransform TargetTransform = TargetMeshComp->GetSocketTransform(IKTargetDef.TargetBoneName);
					
					PDI.DrawLine(SourceTransform.GetLocation(), TargetTransform.GetLocation(), Color, SDPG_Foreground, 0.f);
					SkeletalDebugRendering::DrawAxes(&PDI, FTransform(TargetTransform.GetLocation()), SDPG_Foreground, 0.f, Radius);
				}
				else if (IKTargetDef.Provider == EContextualAnimIKTargetProvider::Autogenerated)
				{
					const FTransform SourceTransform = BindingSkelMeshComp->GetBoneTransform(IKTargetDef.BoneName);

					const FTransform ParentTransform = TargetMeshComp->GetSocketTransform(IKTargetDef.TargetBoneName);
					const FTransform TargetTransform = ViewModel->SceneBindings.GetIKTargetTransformFromBinding(Binding, IKTargetDef.GoalName, Binding.GetAnimMontageTime()) * ParentTransform;

					FVector Start = ParentTransform.GetLocation();
					FVector End = TargetTransform.GetLocation();

					PDI.DrawLine(SourceTransform.GetLocation(), TargetTransform.GetLocation(), Color, SDPG_Foreground, 0.f);
					SkeletalDebugRendering::DrawWireBone(&PDI, Start, End, Color, SDPG_Foreground, Radius);
					SkeletalDebugRendering::DrawAxes(&PDI, FTransform(End), SDPG_Foreground, 0.f, Radius);
				}
			}
		}
	}
}

bool FContextualAnimEdMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (HitProxy != nullptr)
	{
		if (HitProxy->IsA(HActor::StaticGetType()))
		{
			HActor* ActorHitProxy = static_cast<HActor*>(HitProxy);
			ViewModel->UpdateSelection(ActorHitProxy->Actor);
			return true;
		}
		else if (HitProxy->IsA(HSelectionCriterionHitProxy::StaticGetType()))
		{
			HSelectionCriterionHitProxy* CriterionHitProxy = static_cast<HSelectionCriterionHitProxy*>(HitProxy);
			ViewModel->UpdateSelection(CriterionHitProxy->Role, CriterionHitProxy->IndexPair.Key, CriterionHitProxy->IndexPair.Value);
			return true;
		}
	}

	ViewModel->ClearSelection();
	return false; // unhandled
}

bool FContextualAnimEdMode::GetHitResultUnderCursor(FHitResult& OutHitResult, FEditorViewportClient* InViewportClient, const FViewportClick& Click) const
{
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(InViewportClient->Viewport, InViewportClient->GetScene(), InViewportClient->EngineShowFlags).SetRealtimeUpdate(InViewportClient->IsRealtime()));
	FSceneView* View = InViewportClient->CalcSceneView(&ViewFamily);
	FViewportCursorLocation Cursor(View, InViewportClient, Click.GetClickPos().X, Click.GetClickPos().Y);
	const auto ViewportType = InViewportClient->GetViewportType();

	const FVector RayStart = Cursor.GetOrigin();
	const FVector RayEnd = RayStart + Cursor.GetDirection() * HALF_WORLD_MAX;

	return InViewportClient->GetWorld()->LineTraceSingleByChannel(OutHitResult, RayStart, RayEnd, ECC_WorldStatic, FCollisionQueryParams::DefaultQueryParam);
}

bool FContextualAnimEdMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (ViewModel)
	{
		return ViewModel->StartTracking();
	}

	return FEdMode::StartTracking(InViewportClient, InViewport);
}

bool FContextualAnimEdMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	if (ViewModel)
	{
		return ViewModel->EndTracking();
	}

	return FEdMode::EndTracking(InViewportClient, InViewport);
}

bool FContextualAnimEdMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();

	if (CurrentAxis != EAxisList::None)
	{
		if (ViewModel)
		{
			return ViewModel->ProcessInputDelta(InDrag, InRot, InScale);
		}
	}

	return false; // unhandled
}

bool FContextualAnimEdMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	return FEdMode::InputKey(ViewportClient, Viewport, Key, Event);
}

bool FContextualAnimEdMode::AllowWidgetMove()
{
	return ShouldDrawWidget();
}

bool FContextualAnimEdMode::ShouldDrawWidget() const
{
	if (ViewModel)
	{
		return ViewModel->ShouldPreviewSceneDrawWidget();
	}

	return false;
}

bool FContextualAnimEdMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (ViewModel)
	{
		return ViewModel->GetCustomDrawingCoordinateSystem(InMatrix, InData);
	}

	return false;
}

bool FContextualAnimEdMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

FVector FContextualAnimEdMode::GetWidgetLocation() const
{
	if (ViewModel)
	{
		return ViewModel->GetWidgetLocationFromSelection();
	}

	return FVector::ZeroVector;
}
