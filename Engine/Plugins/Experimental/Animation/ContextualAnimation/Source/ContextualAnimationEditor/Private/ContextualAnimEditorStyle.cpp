// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimEditorStyle.h"
#include "Brushes/SlateBoxBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Rendering/SlateRenderer.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr< FSlateStyleSet > FContextualAnimEditorStyle::StyleInstance = NULL;

void FContextualAnimEditorStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FContextualAnimEditorStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FContextualAnimEditorStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ContextualAnimEditorStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef< FSlateStyleSet > FContextualAnimEditorStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("ContextualAnimEditorStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("ContextualAnimation")->GetBaseDir() / TEXT("Content"));

	Style->Set("ContextualAnimEditor.Icon", new IMAGE_BRUSH(TEXT("ButtonIcon_40x"), Icon40x40));

	const FString EngineEditorSlateDir = FPaths::EngineContentDir() / TEXT("Editor/Slate");
	Style->SetContentRoot(EngineEditorSlateDir);
	Style->Set("ContextualAnimEditor.Viewport.Border", new BOX_BRUSH("Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(1.f, 1.f, 1.f, 1.f)));

	return Style;
}

#undef RootToContentDir

void FContextualAnimEditorStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FContextualAnimEditorStyle::Get()
{
	return *StyleInstance;
}
