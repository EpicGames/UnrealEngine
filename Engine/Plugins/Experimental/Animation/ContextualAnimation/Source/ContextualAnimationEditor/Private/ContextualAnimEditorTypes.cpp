// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimEditorTypes.h"
#include "ContextualAnimSceneAsset.h"
#include "PropertyHandle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimEditorTypes)

// UContextualAnimNewIKTargetParams
////////////////////////////////////////////////////////////////////////////////////////////////

void UContextualAnimNewIKTargetParams::Reset(const UContextualAnimSceneAsset& InSceneAsset, const UAnimSequenceBase& InAnimation)
{
	const FContextualAnimTrack* AnimTrack = InSceneAsset.FindAnimTrackByAnimation(&InAnimation);
	check(AnimTrack);

	CachedRoles = InSceneAsset.GetRoles();
	check(CachedRoles.Contains(AnimTrack->Role));

	SceneAssetPtr = &InSceneAsset;

	SectionIdx = AnimTrack->SectionIdx;
	SourceRole = AnimTrack->Role;
	GoalName = NAME_None;
	TargetBone = FBoneReference();
	SourceBone = FBoneReference();

	if (CachedRoles.Num() > 1)
	{
		TargetRole = *CachedRoles.FindByPredicate([this](const FName& Role) { return Role != SourceRole; });
	}
}

bool UContextualAnimNewIKTargetParams::HasValidData() const
{
	return GoalName != NAME_None &&
		TargetBone.BoneName != NAME_None &&
		SourceBone.BoneName != NAME_None &&
		CachedRoles.Contains(TargetRole);
}

const UContextualAnimSceneAsset& UContextualAnimNewIKTargetParams::GetSceneAsset() const
{
	check(SceneAssetPtr.IsValid());
	return *SceneAssetPtr.Get();
}

USkeleton* UContextualAnimNewIKTargetParams::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	bInvalidSkeletonIsError = false;

	FName Role = NAME_None;
	if (PropertyHandle)
	{
		if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UContextualAnimNewIKTargetParams, SourceBone))
		{
			Role = SourceRole;
		}
		else if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UContextualAnimNewIKTargetParams, TargetBone))
		{
			Role = TargetRole;
		}
	}

	USkeleton* Skeleton = nullptr;
	if(Role != NAME_None)
	{
		// Iterate over the AnimTracks until find the first track with animation and pull the skeleton from there.
		GetSceneAsset().ForEachAnimTrack([&Skeleton, Role](const FContextualAnimTrack& AnimTrack)
			{
				if (AnimTrack.Role == Role && AnimTrack.Animation)
				{
					Skeleton = AnimTrack.Animation->GetSkeleton();
					return UE::ContextualAnim::EForEachResult::Break;
				}

				return UE::ContextualAnim::EForEachResult::Continue;
			});
	}

	return Skeleton;
}

TArray<FString> UContextualAnimNewIKTargetParams::GetTargetRoleOptions() const
{
	TArray<FString> Options;

	for (const FName& Role : CachedRoles)
	{
		if (Role != SourceRole)
		{
			Options.Add(Role.ToString());
		}
	}

	return Options;
}
