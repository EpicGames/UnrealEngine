// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimFactory.h"
#include "ContextualAnimSceneAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimFactory)

#define LOCTEXT_NAMESPACE "ContextualAnimFactory"

UContextualAnimFactory::UContextualAnimFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UContextualAnimSceneAsset::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

bool UContextualAnimFactory::CanCreateNew() const
{
	return true;
}

UObject* UContextualAnimFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UObject>(InParent, Class, Name, Flags | RF_Transactional);
}

#undef LOCTEXT_NAMESPACE

