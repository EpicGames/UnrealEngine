// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneNotifySection.h"
#include "ContextualAnimMovieSceneNotifyTrack.h"
#include "Animation/AnimSequenceBase.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimMovieSceneNotifySection)

UContextualAnimMovieSceneNotifyTrack* UContextualAnimMovieSceneNotifySection::GetOwnerTrack() const
{
	return GetTypedOuter<UContextualAnimMovieSceneNotifyTrack>();
}

void UContextualAnimMovieSceneNotifySection::Initialize(const FAnimNotifyEvent& NotifyEvent)
{
	const FFrameRate TickResolution = GetOwnerTrack()->GetTypedOuter<UMovieScene>()->GetTickResolution();
	const FFrameNumber StartFrame = (NotifyEvent.GetTriggerTime() * TickResolution).RoundToFrame();

	FFrameNumber EndFrame;
	if (NotifyEvent.NotifyStateClass)
	{
		EndFrame = (NotifyEvent.GetEndTriggerTime() * TickResolution).RoundToFrame();

	}
	else if (NotifyEvent.Notify)
	{
		// Sequencer Panel doesn't seem to have a way to represent single key events the same way we represent AnimNotify in the Animation Editor.
		// So, for now we represent single notifies as sections with a small fixed interval and mark them as non resizable (see FContextualAnimNotifySection::SectionIsResizable). 
		// This may need to be revisited in the future.

		EndFrame = ((NotifyEvent.GetTriggerTime() + 1.f / 30.f) * TickResolution).RoundToFrame();
	}

	SetRange(TRange<FFrameNumber>::Exclusive(StartFrame, EndFrame));

	AnimNotifyEventGuid = NotifyEvent.Guid;
}

FAnimNotifyEvent* UContextualAnimMovieSceneNotifySection::GetAnimNotifyEvent() const
{
	UAnimSequenceBase& Animation = GetOwnerTrack()->GetAnimation();
	for (FAnimNotifyEvent& NotifyEvent : Animation.Notifies)
	{
		if (NotifyEvent.Guid == AnimNotifyEventGuid)
		{
			return &NotifyEvent;
		}
	}

	return nullptr;
}

UAnimNotifyState* UContextualAnimMovieSceneNotifySection::GetAnimNotifyState() const
{
	UAnimSequenceBase& Animation = GetOwnerTrack()->GetAnimation();
	if(const FAnimNotifyEvent* NotifyEventPtr = GetAnimNotifyEvent())
	{
		return NotifyEventPtr->NotifyStateClass;
	}

	return nullptr;
}
