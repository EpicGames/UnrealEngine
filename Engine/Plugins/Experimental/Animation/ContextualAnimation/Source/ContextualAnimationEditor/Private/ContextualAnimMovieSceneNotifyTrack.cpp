// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneNotifyTrack.h"
#include "ContextualAnimMovieSceneNotifySection.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "ContextualAnimViewModel.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimMovieSceneNotifyTrack)

FContextualAnimViewModel& UContextualAnimMovieSceneNotifyTrack::GetViewModel() const
{
	return GetTypedOuter<UMovieScene>()->GetTypedOuter<UContextualAnimMovieSceneSequence>()->GetViewModel();
}

void UContextualAnimMovieSceneNotifyTrack::Initialize(UAnimSequenceBase& AnimationRef, const FAnimNotifyTrack& NotifyTrack)
{
	// Cache animation this track was created from
	Animation = &AnimationRef;

	SetDisplayName(FText::FromName(NotifyTrack.TrackName));

	for (const FAnimNotifyEvent* NotifyEvent : NotifyTrack.Notifies)
	{
		// Create new movie scene section
		UContextualAnimMovieSceneNotifySection* Section = CastChecked<UContextualAnimMovieSceneNotifySection>(CreateNewSection());

		// Set range and cache guid
		Section->Initialize(*NotifyEvent);

		// Add section to the track
		AddSection(*Section);
	}
}

UAnimSequenceBase& UContextualAnimMovieSceneNotifyTrack::GetAnimation() const
{ 
	check(Animation.IsValid());
	return *Animation.Get();
}

UMovieSceneSection* UContextualAnimMovieSceneNotifyTrack::CreateNewSection()
{
	return NewObject<UContextualAnimMovieSceneNotifySection>(this, NAME_None, RF_Transactional);
}

void UContextualAnimMovieSceneNotifyTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

bool UContextualAnimMovieSceneNotifyTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UContextualAnimMovieSceneNotifySection::StaticClass();;
}

const TArray<UMovieSceneSection*>& UContextualAnimMovieSceneNotifyTrack::GetAllSections() const
{
	return Sections;
}

bool UContextualAnimMovieSceneNotifyTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UContextualAnimMovieSceneNotifyTrack::IsEmpty() const
{
	return Sections.Num() != 0;
}

void UContextualAnimMovieSceneNotifyTrack::RemoveSection(UMovieSceneSection& Section)
{
	// Temp solution to prevent editor from crashing when removing the notify from here while the animation is open in the anim editor
	UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
	check(AssetEditorSubsystem);
	AssetEditorSubsystem->CloseAllEditorsForAsset(&GetAnimation());

	// Remove Notify from the animation
	UContextualAnimMovieSceneNotifySection* NotifySection = CastChecked<UContextualAnimMovieSceneNotifySection>(&Section);

	UAnimSequenceBase& Anim = GetAnimation();

	FGuid Guid = NotifySection->GetAnimNotifyEventGuid();
	const int32 TotalRemoved = Anim.Notifies.RemoveAll([&Guid](const FAnimNotifyEvent& Event) { return Event.Guid == Guid; });
	if (TotalRemoved)
	{
		GetViewModel().AnimationModified(Anim);
	}

	// Remove movie scene section
	Sections.Remove(&Section);
}

void UContextualAnimMovieSceneNotifyTrack::RemoveSectionAt(int32 SectionIndex)
{
	check(Sections.IsValidIndex(SectionIndex));
	RemoveSection(*Sections[SectionIndex]);
}

#if WITH_EDITOR
EMovieSceneSectionMovedResult UContextualAnimMovieSceneNotifyTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	if (Params.MoveType == EPropertyChangeType::ValueSet)
	{
		UContextualAnimMovieSceneNotifySection* NotifySection = CastChecked<UContextualAnimMovieSceneNotifySection>(&Section);

		FFrameRate TickResolution = GetTypedOuter<UMovieScene>()->GetTickResolution();

		if (FAnimNotifyEvent* NotifyEventPtr = NotifySection->GetAnimNotifyEvent())
		{
			UAnimSequenceBase* Anim = nullptr;
			if (NotifyEventPtr->NotifyStateClass)
			{
				Anim = NotifyEventPtr->NotifyStateClass->GetTypedOuter<UAnimSequenceBase>();
			}
			else if (NotifyEventPtr->Notify)
			{
				Anim = NotifyEventPtr->Notify->GetTypedOuter<UAnimSequenceBase>();
			}

			if(Anim)
			{
				const float SectionStartTime = (float)TickResolution.AsSeconds(NotifySection->GetInclusiveStartFrame());
				const float SectionEndTime = (float)TickResolution.AsSeconds(NotifySection->GetExclusiveEndFrame());

				NotifyEventPtr->SetTime(SectionStartTime);
				NotifyEventPtr->SetDuration(SectionEndTime - SectionStartTime);

				GetViewModel().AnimationModified(*Anim);
			}
		}
	}

	return EMovieSceneSectionMovedResult::None;
}
#endif
