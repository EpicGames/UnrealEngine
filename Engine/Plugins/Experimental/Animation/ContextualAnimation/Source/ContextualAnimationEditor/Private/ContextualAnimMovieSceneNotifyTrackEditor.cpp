// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneNotifyTrackEditor.h"
#include "ContextualAnimMovieSceneNotifyTrack.h"
#include "ContextualAnimMovieSceneNotifySection.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimViewModel.h"
#include "DetailsViewArgs.h"
#include "MVVM/Views/ViewUtilities.h"
#include "Framework/Commands/UICommandList.h"
#include "SequencerSectionPainter.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "AnimNotifyState_IKWindow.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "PersonaUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Input/SButton.h"
#include "ContextualAnimEditorTypes.h"
#include "ISequencerModule.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Fonts/FontMeasure.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FContextualAnimMovieSceneNotifyTrackEditor"

// FContextualAnimMovieSceneNotifyTrackEditor
////////////////////////////////////////////////////////////////////////////////////////////////

FContextualAnimMovieSceneNotifyTrackEditor::FContextualAnimMovieSceneNotifyTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
	, NewIKTargetParams(nullptr)
{ 
}

void FContextualAnimMovieSceneNotifyTrackEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (NewIKTargetParams)
	{
		Collector.AddReferencedObject(NewIKTargetParams);
	}
}

UContextualAnimMovieSceneSequence& FContextualAnimMovieSceneNotifyTrackEditor::GetMovieSceneSequence() const
{
	UContextualAnimMovieSceneSequence* Sequence = GetFocusedMovieScene()->GetTypedOuter<UContextualAnimMovieSceneSequence>();
	check(Sequence);
	return *Sequence;
}

TSharedRef<ISequencerTrackEditor> FContextualAnimMovieSceneNotifyTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FContextualAnimMovieSceneNotifyTrackEditor(InSequencer));
}

TSharedRef<ISequencerSection> FContextualAnimMovieSceneNotifyTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	checkf(SectionObject.GetClass()->IsChildOf<UContextualAnimMovieSceneNotifySection>(), TEXT("Unsupported section."));
	return MakeShared<FContextualAnimNotifySection>(SectionObject);
}

void FContextualAnimMovieSceneNotifyTrackEditor::OnInitialize()
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");

	TSharedPtr<FUICommandList> CommandList = MakeShareable(new FUICommandList);

	// Extend the sequencer toolbar to add Back button
	TSharedPtr<FExtensibilityManager> ExtensionManager = SequencerModule.GetToolBarExtensibilityManager();
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
	TSharedPtr<const FExtensionBase> ToolbarExtension = ToolbarExtender->AddToolBarExtension(
		"CurveEditor",
		EExtensionHook::After,
		CommandList,
		FToolBarExtensionDelegate::CreateSP(this, &FContextualAnimMovieSceneNotifyTrackEditor::CustomizeToolBar)
	);
	ExtensionManager->AddExtender(ToolbarExtender);
}

void FContextualAnimMovieSceneNotifyTrackEditor::CustomizeToolBar(FToolBarBuilder& ToolBarBuilder)
{
	ToolBarBuilder.BeginSection("Custom");
	{
		FUIAction Action = FUIAction(FExecuteAction::CreateLambda([this]() {
			GetMovieSceneSequence().GetViewModel().SetDefaultMode();
		}));

		Action.IsActionVisibleDelegate = FIsActionButtonVisible::CreateLambda([this]() {
			return (GetSequencer() != nullptr) ? (GetMovieSceneSequence().GetViewModel().GetTimelineMode() == FContextualAnimViewModel::ETimelineMode::Notifies) : false;
		});

		ToolBarBuilder.AddToolBarButton(
			Action,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.ArrowLeft"));
	}
	ToolBarBuilder.EndSection();
}

void FContextualAnimMovieSceneNotifyTrackEditor::FillNewNotifyMenu(FMenuBuilder& MenuBuilder, bool bIsAnimNotifyState, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex)
{
	// MenuBuilder always has a search widget added to it by default, hence if larger then 1 then something else has been added to it
	if (MenuBuilder.GetMultiBox()->GetBlocks().Num() > 1)
	{
		MenuBuilder.AddMenuSeparator();
	}

	FOnClassPicked OnPicked(FOnClassPicked::CreateLambda([this, Track, RowIndex](UClass* InClass)
		{
			//FSlateApplication::Get().DismissAllMenus();

			CreateNewSection(Track, RowIndex, InClass);
		}
	));

	TSharedRef<SWidget> AnimNotifyStateClassPicker = bIsAnimNotifyState ? PersonaUtils::MakeAnimNotifyStatePicker(&Track->GetAnimation(), OnPicked) : PersonaUtils::MakeAnimNotifyPicker(&Track->GetAnimation(), OnPicked);
	MenuBuilder.AddWidget(AnimNotifyStateClassPicker, FText(), true, false);
}

void FContextualAnimMovieSceneNotifyTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	// Menu that appears when clicking on the Add Track button next to the Search Tracks bar

	if (GetMovieSceneSequence().GetViewModel().GetTimelineMode() == FContextualAnimViewModel::ETimelineMode::Notifies)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("NotifyTrack", "Notify Track"),
			LOCTEXT("NotifyTrackTooltip", "Adds a notify track"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateRaw(this, &FContextualAnimMovieSceneNotifyTrackEditor::AddNewNotifyTrack)
			)
		);
	}
}

TSharedPtr<SWidget> FContextualAnimMovieSceneNotifyTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	// [+Section] button on the track

	check(Track);

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	TWeakObjectPtr<UContextualAnimMovieSceneNotifyTrack> WeakTrack = Cast<UContextualAnimMovieSceneNotifyTrack>(Track);
	const int32 RowIndex = Params.TrackInsertRowIndex;
	auto SubMenuCallback = [this, WeakTrack, RowIndex]
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		UContextualAnimMovieSceneNotifyTrack* TrackPtr = WeakTrack.Get();
		if (TrackPtr)
		{
			//@TODO: Add shortcut for Motion Warping window too and group these options in two sections

			MenuBuilder.AddSubMenu(
				LOCTEXT("AddIKWindow", "Add IK Window"),
				LOCTEXT("AddIKWindowTooltip", "Adds new IK Window"),
				FNewMenuDelegate::CreateRaw(this, &FContextualAnimMovieSceneNotifyTrackEditor::BuildNewIKTargetSubMenu, TrackPtr, RowIndex));

			MenuBuilder.AddSubMenu(
				LOCTEXT("AddNotify", "Add Notify"),
				LOCTEXT("AddNotifyToolTip", "Adds new AnimNotify"),
				FNewMenuDelegate::CreateRaw(this, &FContextualAnimMovieSceneNotifyTrackEditor::FillNewNotifyMenu, false /*bIsAnimNotifyState*/, TrackPtr, RowIndex));

			MenuBuilder.AddSubMenu(
				LOCTEXT("AddNotifyState", "Add Notify State"),
				LOCTEXT("AddNotifyStateToolTip", "Adds new AnimNotifyState"),
				FNewMenuDelegate::CreateRaw(this, &FContextualAnimMovieSceneNotifyTrackEditor::FillNewNotifyMenu, true /*bIsAnimNotifyState*/, TrackPtr, RowIndex));
		}
		else
		{
			MenuBuilder.AddWidget(SNew(STextBlock).Text(LOCTEXT("InvalidTrack", "Track is no longer valid")), FText(), true);
		}

		return MenuBuilder.MakeWidget();
	};

	return UE::Sequencer::MakeAddButton(
		LOCTEXT("Notify", "Notify"),
		FOnGetContent::CreateLambda(SubMenuCallback),
		Params.ViewModel);
}

void FContextualAnimMovieSceneNotifyTrackEditor::BuildNewIKTargetSubMenu(FMenuBuilder& MenuBuilder, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex)
{
	// Create new IK Target
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("CreateNewIKTarget", "Create new IK Target"));

		MenuBuilder.AddSubMenu(
			LOCTEXT("NewIKTarget", "New IK Target"),
			LOCTEXT("NewIKTargetTooltip", "Creates a new IK Target and adds an IK window for it"),
			FNewMenuDelegate::CreateRaw(this, &FContextualAnimMovieSceneNotifyTrackEditor::BuildNewIKTargetWidget, Track, RowIndex),
			false,
			FSlateIcon()
		);

		MenuBuilder.EndSection();
	}

	// List of IK Targets already created
	{
		MenuBuilder.BeginSection(NAME_None, LOCTEXT("AvailableIKTargets", "Available IK Targets"));

		const UContextualAnimSceneAsset* SceneAsset = GetMovieSceneSequence().GetViewModel().GetSceneAsset();
		const FContextualAnimTrack* AnimTrack = SceneAsset->FindAnimTrackByAnimation(&Track->GetAnimation());
		check(AnimTrack);

		const TArray<FContextualAnimIKTargetDefinition>& IKTargetDefs = SceneAsset->GetIKTargetDefsForRole(AnimTrack->Role).IKTargetDefs;
		for (const FContextualAnimIKTargetDefinition& IKTargetDef : IKTargetDefs)
		{
			const FName GoalName = IKTargetDef.GoalName;
			MenuBuilder.AddMenuEntry(
 			FText::FromName(GoalName),
			FText::GetEmpty(),
 			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([this, Track, RowIndex, GoalName]() {
					CreateNewIKSection(Track, RowIndex, GoalName);
			})));
		}

		MenuBuilder.EndSection();
	}
}

UContextualAnimMovieSceneNotifySection* FContextualAnimMovieSceneNotifyTrackEditor::CreateNewIKSection(UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex, const FName& GoalName)
{
	UContextualAnimMovieSceneNotifySection* NewSection = CreateNewSection(Track, RowIndex, UAnimNotifyState_IKWindow::StaticClass());
	check(NewSection);

	UAnimNotifyState_IKWindow* AnimNotify = Cast<UAnimNotifyState_IKWindow>(NewSection->GetAnimNotifyState());
	check(AnimNotify);
 
	AnimNotify->GoalName = GoalName;

	return NewSection;
}

void FContextualAnimMovieSceneNotifyTrackEditor::BuildNewIKTargetWidget(FMenuBuilder& MenuBuilder, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex)
{
	if(NewIKTargetParams == nullptr)
	{
		NewIKTargetParams = NewObject<UContextualAnimNewIKTargetParams>(UContextualAnimNewIKTargetParams::StaticClass());
	}
	
	check(NewIKTargetParams);

	NewIKTargetParams->Reset(*GetMovieSceneSequence().GetViewModel().GetSceneAsset(), Track->GetAnimation());

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bAllowSearch = false;
	Args.bAllowFavoriteSystem = false;

	TSharedPtr<IDetailsView> Widget = PropertyModule.CreateDetailView(Args);
	Widget->SetObject(NewIKTargetParams);

	MenuBuilder.AddWidget(
		SNew(SBox)
		.MinDesiredWidth(500.0f)
		.MaxDesiredWidth(500.f)
		.MaxDesiredHeight(400.0f)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				Widget.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			[
				SNew(SButton)
				.ContentPadding(3)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.IsEnabled_Lambda([this](){ return NewIKTargetParams->HasValidData(); })
				.OnClicked_Lambda([this, Track, RowIndex]()
				{
					GetMovieSceneSequence().GetViewModel().AddNewIKTarget(*NewIKTargetParams);

					// Create IK section for the newly created target
	 				CreateNewIKSection(Track, RowIndex, NewIKTargetParams->GoalName);

					FSlateApplication::Get().DismissAllMenus();

					return FReply::Handled();
				})
			.Text(LOCTEXT("OK", "OK"))
			]
		],
		FText(), true, false);
}

void FContextualAnimMovieSceneNotifyTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{	
	// Builds menu that appears when clicking on the +Track button on an Object Track
}

void FContextualAnimMovieSceneNotifyTrackEditor::AddNewNotifyTrack()
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr.IsValid())
	{
		return;
	}

	// Copied from AnimTimelineTrack_Notifies.cpp/FAnimTimelineTrack_Notifies::GetNewTrackName()
	auto GetNewTrackName = [](UAnimSequenceBase* InAnimSequenceBase) -> FName
	{
		TArray<FName> TrackNames;
		TrackNames.Reserve(50);

		for (const FAnimNotifyTrack& Track : InAnimSequenceBase->AnimNotifyTracks)
		{
			TrackNames.Add(Track.TrackName);
		}

		FName NameToTest;
		int32 TrackIndex = 1;

		do
		{
			NameToTest = *FString::FromInt(TrackIndex++);
		} while (TrackNames.Contains(NameToTest));

		return NameToTest;
	};

	UAnimSequenceBase* Animation = GetMovieSceneSequence().GetViewModel().GetEditingAnimation();
	check(Animation);

	// Copied from AnimTimelineTrack_Notifies.cpp/FAnimTimelineTrack_Notifies::AddTrack()

	FAnimNotifyTrack NewNotifyTrack;
	NewNotifyTrack.TrackName = GetNewTrackName(Animation);
	NewNotifyTrack.TrackColor = FLinearColor::White;

	Animation->AnimNotifyTracks.Add(NewNotifyTrack);

	GetMovieSceneSequence().GetViewModel().AnimationModified(*Animation);

	// Create and Initialize MovieSceneTrack
	UContextualAnimMovieSceneNotifyTrack* MovieSceneTrack = GetMovieSceneSequence().GetMovieScene()->AddTrack<UContextualAnimMovieSceneNotifyTrack>();
	check(MovieSceneTrack);

	MovieSceneTrack->Initialize(*Animation, NewNotifyTrack);

	SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);
}

bool FContextualAnimMovieSceneNotifyTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UContextualAnimMovieSceneNotifyTrack::StaticClass());
}

bool FContextualAnimMovieSceneNotifyTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UContextualAnimMovieSceneNotifyTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

UContextualAnimMovieSceneNotifySection* FContextualAnimMovieSceneNotifyTrackEditor::CreateNewSection(UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex, UClass* NotifyClass)
{
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (SequencerPtr.IsValid())
	{
		UAnimSequenceBase& Animation = Track->GetAnimation();

		const FName TrackName = FName(Track->GetDisplayName().ToString());
		const int32 TrackIndex = Animation.AnimNotifyTracks.IndexOfByPredicate([TrackName](const FAnimNotifyTrack& AnimNotifyTrack) { return AnimNotifyTrack.TrackName == TrackName; });
		check(TrackIndex != INDEX_NONE);

		// Add new AnimNotifyEvent to the animation
		FAnimNotifyEvent& NewEvent = Animation.Notifies.AddDefaulted_GetRef();
		{
			NewEvent.Guid = FGuid::NewGuid();

			const float StartTime = 0.f;
			NewEvent.Link(&Animation, StartTime);
			NewEvent.TriggerTimeOffset = GetTriggerTimeOffsetForType(Animation.CalculateOffsetForNotify(StartTime));
			NewEvent.TrackIndex = TrackIndex;

			if (NotifyClass)
			{
				UObject* AnimNotifyClass = NewObject<UObject>(&Animation, NotifyClass, NAME_None, RF_Transactional);
				NewEvent.NotifyStateClass = Cast<UAnimNotifyState>(AnimNotifyClass);
				NewEvent.Notify = Cast<UAnimNotify>(AnimNotifyClass);

				if (NewEvent.NotifyStateClass)
				{
					// Set default duration to 1 frame for AnimNotifyState.
					NewEvent.SetDuration(1 / 30.f);
					NewEvent.EndLink.Link(&Animation, NewEvent.EndLink.GetTime());
					NewEvent.TriggerWeightThreshold = NewEvent.NotifyStateClass->GetDefaultTriggerWeightThreshold();
					NewEvent.NotifyName = FName(*NewEvent.NotifyStateClass->GetNotifyName());
				}
				else if (NewEvent.Notify)
				{
					NewEvent.TriggerWeightThreshold = NewEvent.Notify->GetDefaultTriggerWeightThreshold();
					NewEvent.NotifyName = FName(*NewEvent.Notify->GetNotifyName());
				}
			}
			else
			{
				NewEvent.Notify = nullptr;
				NewEvent.NotifyStateClass = nullptr;
			}
		}

		// Create new movie scene section
		UContextualAnimMovieSceneNotifySection* NewSection = CastChecked<UContextualAnimMovieSceneNotifySection>(Track->CreateNewSection());

		NewSection->SetRowIndex(RowIndex);

		// Set range and cache guid
		NewSection->Initialize(NewEvent);
	
		GetMovieSceneSequence().GetViewModel().AnimationModified(Animation);

		// Add section to the track
		Track->AddSection(*NewSection);
		Track->UpdateEasing();
		Track->Modify();

		// Select new section
		SequencerPtr->EmptySelection();
		SequencerPtr->SelectSection(NewSection);
		SequencerPtr->ThrobSectionSelection();

		SequencerPtr->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemAdded);

		return NewSection;
	}

	return nullptr;
}

// FContextualAnimNotifySection
////////////////////////////////////////////////////////////////////////////////////////////////

UContextualAnimMovieSceneNotifySection* FContextualAnimNotifySection::GetNotifySection() const
{
	if (UContextualAnimMovieSceneNotifySection* NotifySection = Cast<UContextualAnimMovieSceneNotifySection>(WeakSection.Get()))
	{
		return NotifySection;
	}

	return nullptr;
}

void FContextualAnimNotifySection::PaintNotifyName(FSequencerSectionPainter& Painter, int32 LayerId, const FString& InEventString, float PixelPos, bool bIsEventValid)
{
	static const int32   FontSize = 10;
	static const float   BoxOffsetPx = 10.f;
	static const TCHAR* WarningString = TEXT("\xf071");

	const FSlateFontInfo FontAwesomeFont = FAppStyle::Get().GetFontStyle("FontAwesome.10");
	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Bold", 10);
	const FLinearColor   DrawColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Setup the warning size. Static since it won't ever change
	static FVector2D WarningSize = FontMeasureService->Measure(WarningString, FontAwesomeFont);
	const  FMargin   WarningPadding = (bIsEventValid || InEventString.Len() == 0) ? FMargin(0.f) : FMargin(0.f, 0.f, 4.f, 0.f);
	const  FMargin   BoxPadding = FMargin(4.0f, 2.0f);

	const FVector2D  TextSize = FontMeasureService->Measure(InEventString, SmallLayoutFont);
	const FVector2D  IconSize = bIsEventValid ? FVector2D::ZeroVector : WarningSize;
	const FVector2D  PaddedIconSize = IconSize + WarningPadding.GetDesiredSize();
	const FVector2D  BoxSize = FVector2D(TextSize.X + PaddedIconSize.X, FMath::Max(TextSize.Y, PaddedIconSize.Y)) + BoxPadding.GetDesiredSize();

	FVector2D BoxOffset = FVector2D(PixelPos + BoxOffsetPx, Painter.SectionGeometry.Size.Y * .5f - BoxSize.Y * .5f);
	FVector2D IconOffset = FVector2D(BoxPadding.Left, BoxSize.Y * .5f - IconSize.Y * .5f);
	FVector2D TextOffset = FVector2D(IconOffset.X + PaddedIconSize.X, BoxSize.Y * .5f - TextSize.Y * .5f);

	// Draw the background box
	FSlateDrawElement::MakeBox(
		Painter.DrawElements,
		LayerId + 1,
		Painter.SectionGeometry.ToPaintGeometry(BoxSize, FSlateLayoutTransform(BoxOffset)),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor::Black.CopyWithNewOpacity(0.5f)
	);

	if (!bIsEventValid)
	{
		// Draw a warning icon for unbound repeaters
		FSlateDrawElement::MakeText(
			Painter.DrawElements,
			LayerId + 2,
			Painter.SectionGeometry.ToPaintGeometry(IconSize, FSlateLayoutTransform(BoxOffset + IconOffset)),
			WarningString,
			FontAwesomeFont,
			Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
			FAppStyle::GetWidgetStyle<FTextBlockStyle>("Log.Warning").ColorAndOpacity.GetSpecifiedColor()
		);
	}

	FSlateDrawElement::MakeText(
		Painter.DrawElements,
		LayerId + 2,
		Painter.SectionGeometry.ToPaintGeometry(TextSize, FSlateLayoutTransform(BoxOffset + TextOffset)),
		InEventString,
		SmallLayoutFont,
		Painter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect,
		DrawColor
	);
}

int32 FContextualAnimNotifySection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	int32 LayerId = InPainter.PaintSectionBackground();

	// Pop current clip state since we want the notify name to 'overflow' the geometry of the section
	TOptional<FSlateClippingState> PreviousClipState = InPainter.DrawElements.GetClippingState();
	InPainter.DrawElements.PopClip();

	// Draw notify name
	FContextualAnimNotifySection::PaintNotifyName(InPainter, LayerId, GetNotifyName(), 0, true);

	// Restore clip state
	if (PreviousClipState.IsSet())
	{
		InPainter.DrawElements.GetClippingManager().PushClippingState(PreviousClipState.GetValue());
	}

	return LayerId;
}

bool FContextualAnimNotifySection::SectionIsResizable() const
{
	if (UContextualAnimMovieSceneNotifySection* NotifySection = GetNotifySection())
	{
		const FAnimNotifyEvent* AnimNotifyEvent = NotifySection->GetAnimNotifyEvent();
		return (AnimNotifyEvent && AnimNotifyEvent->NotifyStateClass);
	}

	return false;
}

FString FContextualAnimNotifySection::GetNotifyName() const
{
	if (const UContextualAnimMovieSceneNotifySection* NotifySection = GetNotifySection())
	{
		const FAnimNotifyEvent* AnimNotifyEvent = NotifySection->GetAnimNotifyEvent();
		if (AnimNotifyEvent)
		{
			if (AnimNotifyEvent->NotifyStateClass)
			{
				return AnimNotifyEvent->NotifyStateClass->GetNotifyName();
			}
			else if (AnimNotifyEvent->Notify)
			{
				return AnimNotifyEvent->Notify->GetNotifyName();
			}
		}
	}

	return FString(TEXT("Invalid"));
}

#undef LOCTEXT_NAMESPACE
