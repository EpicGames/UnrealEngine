// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneSection.h"
#include "ContextualAnimMovieSceneTrack.h"
#include "ContextualAnimViewModel.h"
#include "ContextualAnimSceneAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimMovieSceneSection)

UContextualAnimMovieSceneTrack& UContextualAnimMovieSceneSection::GetOwnerTrack() const
{
	UContextualAnimMovieSceneTrack* OwnerTrackPtr = GetTypedOuter<UContextualAnimMovieSceneTrack>();
	check(OwnerTrackPtr);

	return *OwnerTrackPtr;
}

void UContextualAnimMovieSceneSection::Initialize(int32 InSectionIdx, int32 InAnimSetIdx, int32 InAnimTrackIdx, int32 InSegmentIdx)
{
	SectionIdx = InSectionIdx;
	AnimSetIdx = InAnimSetIdx;
	AnimTrackIdx = InAnimTrackIdx;
	SegmentIdx = InSegmentIdx;
}

FContextualAnimTrack& UContextualAnimMovieSceneSection::GetAnimTrack() const
{
	const FContextualAnimTrack* AnimTrack = GetOwnerTrack().GetViewModel().GetSceneAsset()->GetAnimTrack(SectionIdx, AnimSetIdx, AnimTrackIdx);
	check(AnimTrack);

	return *const_cast<FContextualAnimTrack*>(AnimTrack);
}
