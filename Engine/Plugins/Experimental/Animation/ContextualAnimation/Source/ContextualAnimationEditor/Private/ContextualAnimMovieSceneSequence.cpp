// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneSequence.h"
#include "ContextualAnimMovieSceneNotifyTrack.h"
#include "ContextualAnimMovieSceneTrack.h"
#include "ContextualAnimViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimMovieSceneSequence)

UContextualAnimMovieSceneSequence::UContextualAnimMovieSceneSequence(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

ETrackSupport UContextualAnimMovieSceneSequence::IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const
{ 
	if (InTrackClass == UContextualAnimMovieSceneNotifyTrack::StaticClass() || 
		InTrackClass == UContextualAnimMovieSceneTrack::StaticClass())
	{
		return ETrackSupport::Supported;
	}
	
	return Super::IsTrackSupportedImpl(InTrackClass);
}

void UContextualAnimMovieSceneSequence::Initialize(const TSharedRef<FContextualAnimViewModel>& ViewModelRef)
{
	ViewModelPtr = ViewModelRef;
}

void UContextualAnimMovieSceneSequence::BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context)
{
	if (AActor* Actor = Cast<AActor>(&PossessedObject))
	{
		BoundActors.Add(ObjectId, Actor);
	}
}

bool UContextualAnimMovieSceneSequence::CanPossessObject(UObject& Object, UObject* InPlaybackContext) const
{
	return true;
}

void UContextualAnimMovieSceneSequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	const TWeakObjectPtr<AActor>* WeakActorPtr = BoundActors.Find(ObjectId);
	if (WeakActorPtr && WeakActorPtr->IsValid())
	{
		OutObjects.Add(WeakActorPtr->Get());
	}
}

UObject* UContextualAnimMovieSceneSequence::GetParentObject(UObject* Object) const
{
	return nullptr;
}

void UContextualAnimMovieSceneSequence::UnbindPossessableObjects(const FGuid& ObjectId)
{
	BoundActors.Remove(ObjectId);
}

void UContextualAnimMovieSceneSequence::UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context)
{
	BoundActors.Remove(ObjectId);
}

void UContextualAnimMovieSceneSequence::UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context)
{
	BoundActors.Remove(ObjectId);
}

UMovieScene* UContextualAnimMovieSceneSequence::GetMovieScene() const
{
	UMovieScene* MovieScene = GetViewModel().GetMovieScene();
	checkf(MovieScene, TEXT("ContextualAnim sequence not initialized"));
	return MovieScene;
}
