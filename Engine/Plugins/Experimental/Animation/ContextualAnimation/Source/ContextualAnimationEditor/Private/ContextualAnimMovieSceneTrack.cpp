// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneTrack.h"
#include "ContextualAnimMovieSceneSection.h"
#include "ContextualAnimMovieSceneSequence.h"
#include "MovieScene.h"
#include "ContextualAnimTypes.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimViewModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContextualAnimMovieSceneTrack)

#define LOCTEXT_NAMESPACE "ContexualAnimTrack"

UContextualAnimMovieSceneTrack::UContextualAnimMovieSceneTrack()
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(110, 155, 57, 150);
#endif
}

void UContextualAnimMovieSceneTrack::Initialize(const FName& InRole)
{
	Role = InRole;

	SetDisplayName(FText::FromString(FString::Printf(TEXT("Role: %s"), *Role.ToString())));
}

FContextualAnimViewModel& UContextualAnimMovieSceneTrack::GetViewModel() const
{
	return GetTypedOuter<UMovieScene>()->GetTypedOuter<UContextualAnimMovieSceneSequence>()->GetViewModel();
}

UMovieSceneSection* UContextualAnimMovieSceneTrack::CreateNewSection()
{
	return NewObject<UContextualAnimMovieSceneSection>(this, NAME_None, RF_Transactional);
}

void UContextualAnimMovieSceneTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

bool UContextualAnimMovieSceneTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UContextualAnimMovieSceneSection::StaticClass();;
}

const TArray<UMovieSceneSection*>& UContextualAnimMovieSceneTrack::GetAllSections() const
{
	return Sections;
}

bool UContextualAnimMovieSceneTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UContextualAnimMovieSceneTrack::IsEmpty() const
{
	return Sections.Num() != 0;
}

void UContextualAnimMovieSceneTrack::RemoveSection(UMovieSceneSection& Section)
{
	// Remove movie scene section
	Sections.Remove(&Section);
}

void UContextualAnimMovieSceneTrack::RemoveSectionAt(int32 SectionIndex)
{
	check(Sections.IsValidIndex(SectionIndex));
	RemoveSection(*Sections[SectionIndex]);
}

#if WITH_EDITOR
EMovieSceneSectionMovedResult UContextualAnimMovieSceneTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	if (Params.MoveType == EPropertyChangeType::ValueSet)
	{
		UContextualAnimMovieSceneSection* NotifySection = CastChecked<UContextualAnimMovieSceneSection>(&Section);

		const FContextualAnimTrack& AnimTrack = NotifySection->GetAnimTrack();
		if(AnimTrack.Animation)
		{
			const float AnimLength = AnimTrack.Animation->GetPlayLength();
			const FFrameRate TickResolution = GetTypedOuter<UMovieScene>()->GetTickResolution();
			NotifySection->SetRange(TRange<FFrameNumber>::Inclusive(0, (AnimLength * TickResolution).RoundToFrame()));
		}
	}

	return EMovieSceneSectionMovedResult::None;
}

FSlateColor UContextualAnimMovieSceneTrack::GetLabelColor(const FMovieSceneLabelParams& LabelParams) const
{
	if (Role != NAME_None)
	{
		if (const UContextualAnimSceneAsset* SceneAsset = GetViewModel().GetSceneAsset())
		{
			const TArray<FName> ValidRoles = SceneAsset->GetRoles();
			if (ValidRoles.Contains(Role) == false)
			{
				return FSlateColor(FColor::Red);
			}
		}
	}

	return Super::GetLabelColor(LabelParams);
}

FText UContextualAnimMovieSceneTrack::GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const
{
	if (Role != NAME_None)
	{
		if (const UContextualAnimSceneAsset* SceneAsset = GetViewModel().GetSceneAsset())
		{
			const TArray<FName> ValidRoles = SceneAsset->GetRoles();
			if (ValidRoles.Contains(Role) == false)
			{
				return FText::Format(LOCTEXT("ContextualAnimMovieSceneTrack_MissingRole", "Role {0} is missing from Role Asset {1}. Add the role to the Roles Asset, or Update Roles in the details panel."), 
					FText::FromName(Role), FText::FromString(GetNameSafe(SceneAsset)));
			}
		}
	}


	return Super::GetDisplayNameToolTipText(LabelParams);
}

#endif

#undef LOCTEXT_NAMESPACE