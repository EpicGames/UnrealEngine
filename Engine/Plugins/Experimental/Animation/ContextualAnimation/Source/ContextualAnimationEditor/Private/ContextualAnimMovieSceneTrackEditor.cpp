// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimMovieSceneTrackEditor.h"
#include "ContextualAnimMovieSceneTrack.h"
#include "ContextualAnimMovieSceneSection.h"
#include "ContextualAnimViewModel.h"
#include "SequencerSectionPainter.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "FContextualAnimMovieSceneTrackEditor"

// FContextualAnimMovieSceneTrackEditor
////////////////////////////////////////////////////////////////////////////////////////////////

FContextualAnimMovieSceneTrackEditor::FContextualAnimMovieSceneTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{ 
}

TSharedRef<ISequencerTrackEditor> FContextualAnimMovieSceneTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FContextualAnimMovieSceneTrackEditor(InSequencer));
}

TSharedRef<ISequencerSection> FContextualAnimMovieSceneTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	checkf(SectionObject.GetClass()->IsChildOf<UContextualAnimMovieSceneSection>(), TEXT("Unsupported section."));
	return MakeShared<FContextualAnimSection>(SectionObject);
}

void FContextualAnimMovieSceneTrackEditor::BuildAddTrackMenu(FMenuBuilder& MenuBuilder)
{
	// Menu that appears when clicking on the Add Track button next to the Search Tracks bar
}

TSharedPtr<SWidget> FContextualAnimMovieSceneTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	// [+Section] button on the track

	return SNullWidget::NullWidget;
}

void FContextualAnimMovieSceneTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{	
	// Builds menu that appears when clicking on the +Track button on an Object Track
}

bool FContextualAnimMovieSceneTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return (Type == UContextualAnimMovieSceneTrack::StaticClass());
}

bool FContextualAnimMovieSceneTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UContextualAnimMovieSceneTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported == ETrackSupport::Supported;
}

// FContextualAnimSection
////////////////////////////////////////////////////////////////////////////////////////////////

int32 FContextualAnimSection::OnPaintSection(FSequencerSectionPainter& InPainter) const
{
	int32 LayerId = FSequencerSection::OnPaintSection(InPainter);

	if (UContextualAnimMovieSceneSection* Section = Cast<UContextualAnimMovieSceneSection>(WeakSection.Get()))
	{
		const FContextualAnimViewModel& ViewModel = Section->GetOwnerTrack().GetViewModel();

		if (const FContextualAnimTrack* SelectedTrack = ViewModel.GetSelectedAnimTrack())
		{
			if (Section->GetSectionIdx() == SelectedTrack->SectionIdx &&
				Section->GetAnimSetIdx() == SelectedTrack->AnimSetIdx &&
				Section->GetAnimTrackIdx() == SelectedTrack->AnimTrackIdx)
			{
				const ESlateDrawEffect DrawEffects = InPainter.bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

				FSlateDrawElement::MakeBox(
					InPainter.DrawElements,
					LayerId,
					InPainter.SectionGeometry.ToPaintGeometry(),
					FAppStyle::GetBrush(TEXT("Sequencer.Section.LockedBorder")),
					DrawEffects,
					FStyleColors::AccentGreen.GetColor(FWidgetStyle())
				);
				++LayerId;
			}
		}
	}

	return LayerId;
}

FText FContextualAnimSection::GetSectionTitle() const
{
	if (UContextualAnimMovieSceneSection* Section = Cast<UContextualAnimMovieSceneSection>(WeakSection.Get()))
	{
		if(Section->GetAnimTrack().Animation)
		{
			return FText::FromString(GetNameSafe(Section->GetAnimTrack().Animation));
		}
		else
		{
			return FText::FromString(TEXT("No Animation"));
		}
	}

	return FText::FromString(TEXT("Animation_Name"));
}

FReply FContextualAnimSection::OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent) 
{ 
	if (UContextualAnimMovieSceneSection* Section = Cast<UContextualAnimMovieSceneSection>(WeakSection.Get()))
	{
		if(Section->IsActive())
		{
			FContextualAnimTrack& AnimTrack = Section->GetAnimTrack();
			if(AnimTrack.Animation)
			{
				Section->GetOwnerTrack().GetViewModel().SetNotifiesMode(Section->GetAnimTrack());
			}
		}
		else
		{
			Section->GetOwnerTrack().GetViewModel().SetActiveAnimSetForSection(Section->GetSectionIdx(), Section->GetAnimSetIdx());
		}

		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

float FContextualAnimSection::GetSectionHeight() const
{
	return 28.f;
}

#undef LOCTEXT_NAMESPACE
