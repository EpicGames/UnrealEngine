// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimPreviewScene.h"
#include "Components/StaticMeshComponent.h"
#include "ContextualAnimAssetEditorToolkit.h"
#include "EngineUtils.h"

FContextualAnimPreviewScene::FContextualAnimPreviewScene(ConstructionValues CVS, const TSharedRef<FContextualAnimAssetEditorToolkit>& EditorToolkit)
	: FAdvancedPreviewScene(CVS)
	, EditorToolkitPtr(EditorToolkit)
{
	// Disable killing actors outside of the world
	AWorldSettings* WorldSettings = GetWorld()->GetWorldSettings(true);
	WorldSettings->bEnableWorldBoundsChecks = false;

	// Spawn an owner for FloorMeshComponent so CharacterMovementComponent can detect it as a valid floor and slide along it
	{
		AActor* FloorActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass(), FTransform());
		check(FloorActor);

		static const FString NewName = FString(TEXT("FloorComponent"));
		FloorMeshComponent->Rename(*NewName, FloorActor);

		FloorActor->SetRootComponent(FloorMeshComponent);
	}
}

void FContextualAnimPreviewScene::Tick(float InDeltaTime)
{
	FAdvancedPreviewScene::Tick(InDeltaTime);

	GetWorld()->Tick(LEVELTICK_All, InDeltaTime);
}
