// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimTypeActions.h"
#include "ContextualAnimAssetEditorToolkit.h"
#include "ContextualAnimSceneAsset.h"

#define LOCTEXT_NAMESPACE "ContextualAnimTypeActions"

FText FContextualAnimTypeActions::GetName() const
{
	return LOCTEXT("ContextualAnimTypeActionsName", "Contextual Anim Scene");
}

FColor FContextualAnimTypeActions::GetTypeColor() const
{
	return FColor(129, 196, 115);
}

UClass* FContextualAnimTypeActions::GetSupportedClass() const
{
	return UContextualAnimSceneAsset::StaticClass();
}

void FContextualAnimTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UContextualAnimSceneAsset* Asset = Cast<UContextualAnimSceneAsset>(*ObjIt))
		{
			TSharedRef<FContextualAnimAssetEditorToolkit> NewEditor(new FContextualAnimAssetEditorToolkit());
			NewEditor->InitAssetEditor(Mode, EditWithinLevelEditor, Asset);
		}
	}
}

uint32 FContextualAnimTypeActions::GetCategories()
{
	return EAssetTypeCategories::Animation;
}

#undef LOCTEXT_NAMESPACE