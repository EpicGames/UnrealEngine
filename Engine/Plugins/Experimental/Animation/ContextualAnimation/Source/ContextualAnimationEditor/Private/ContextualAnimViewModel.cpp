// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimViewModel.h"
#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "IDetailsView.h"
#include "ISequencerModule.h"
#include "ContextualAnimMovieSceneTrack.h"
#include "ContextualAnimMovieSceneSection.h"
#include "ContextualAnimMovieSceneNotifyTrack.h"
#include "ContextualAnimOverrideInterface.h"
#include "ContextualAnimActorInterface.h"
#include "ContextualAnimPreviewScene.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "ContextualAnimEditorTypes.h"
#include "ContextualAnimUtilities.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimSceneActorComponent.h"
#include "PropertyEditorModule.h"
#include "Misc/TransactionObjectEvent.h" // IWYU pragma: keep
#include "Modules/ModuleManager.h"
#include "GameFramework/Character.h"
#include "GameFramework/CharacterMovementComponent.h"
#include "Components/CapsuleComponent.h"
#include "MotionWarpingComponent.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimPreviewInstance.h"
#include "ContextualAnimSelectionCriterion.h"
#include "ScopedTransaction.h"
#include "Misc/MessageDialog.h"
#include "UObject/Package.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/StaticMesh.h"
#include "Components/StaticMeshComponent.h"
#include "DetailCustomizations/ContextualAnimSceneAssetDetailCustom.h"

#define LOCTEXT_NAMESPACE "ContextualAnimViewModel"

static const FName PreviewActorTag = FName(TEXT("__PreviewActor__"));

static void ShowMissingSceneActorComponentWarning(const FString& ClassName)
{
	const FText WarningMsg = FText::Format(
		LOCTEXT("MissingSceneActorComponent",
			"The PreviewActorClass '{0}' does not contain a UContextualAnimSceneActorComponent. "
			"This component is required for the Contextual Animation editor to function correctly. "
			"The preview actor selection has been reverted."),
		FText::FromString(ClassName));

	FMessageDialog::Open(EAppMsgType::Ok, WarningMsg);
}

/** Returns true if the underlying animation sequence at the given time has bForceRootLock enabled. */
static bool HasForceRootLock(const UAnimSequenceBase* Animation, float Time)
{
	if (const UAnimMontage* Montage = Cast<UAnimMontage>(Animation))
	{
		if (const FAnimSegment* Segment = Montage->SlotAnimTracks[0].AnimTrack.GetSegmentAtTime(Time))
		{
			if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Segment->GetAnimReference()))
			{
				return AnimSequence->bForceRootLock;
			}
		}
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Animation))
	{
		return AnimSequence->bForceRootLock;
	}
	return false;
}

FContextualAnimViewModel::FContextualAnimViewModel()
	: SceneAsset(nullptr)
	, MovieSceneSequence(nullptr)
	, MovieScene(nullptr)
{
}

FContextualAnimViewModel::~FContextualAnimViewModel()
{
	check(!bInitialized)
}

void FContextualAnimViewModel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SceneAsset);
	Collector.AddReferencedObject(MovieSceneSequence);
	Collector.AddReferencedObject(MovieScene);
}

TSharedPtr<ISequencer> FContextualAnimViewModel::GetSequencer()
{
	return Sequencer;
}

void FContextualAnimViewModel::Initialize(UContextualAnimSceneAsset* InSceneAsset, const TSharedRef<FContextualAnimPreviewScene>& InPreviewScene)
{
	check(!bInitialized);

	SceneAsset = InSceneAsset;
	PreviewScenePtr = InPreviewScene;

	CreateSequencer();

	CreateDetailsView();

	SetDefaultMode();

	bInitialized = true;
}

void FContextualAnimViewModel::Shutdown()
{
	check(DetailsView.IsValid())
	DetailsView->UnregisterInstancedCustomPropertyLayout(UContextualAnimSceneAsset::StaticClass());
	DetailsView.Reset();

	check(Sequencer.IsValid());
	Sequencer->OnMovieSceneDataChanged().RemoveAll(this);
	Sequencer->OnGlobalTimeChanged().RemoveAll(this);
	Sequencer->OnPlayEvent().RemoveAll(this);
	Sequencer->OnStopEvent().RemoveAll(this);
	Sequencer.Reset();

	bInitialized = false;
}

bool FContextualAnimViewModel::CanMakeEdits() const
{
	return IsSimulateModeInactive();
}

void FContextualAnimViewModel::CreateDetailsView()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;

	DetailsView = PropertyModule.CreateDetailView(Args);
	DetailsView->SetObject(SceneAsset);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FContextualAnimViewModel::OnFinishedChangingProperties);
	DetailsView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateSP(this, &FContextualAnimViewModel::CanMakeEdits));
	DetailsView->RegisterInstancedCustomPropertyLayout(UContextualAnimSceneAsset::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FContextualAnimSceneAssetDetailCustom::MakeInstance, AsShared()));
}

void FContextualAnimViewModel::CreateSequencer()
{
	MovieSceneSequence = NewObject<UContextualAnimMovieSceneSequence>(GetTransientPackage());
	MovieSceneSequence->Initialize(AsShared());

	MovieScene = NewObject<UMovieScene>(MovieSceneSequence, FName("ContextualAnimMovieScene"), RF_Transactional);
	MovieScene->SetDisplayRate(FFrameRate(30, 1));

	FSequencerViewParams ViewParams(TEXT("ContextualAnimSequenceSettings"));
	{
		ViewParams.UniqueName = "ContextualAnimSequenceEditor";
		//ViewParams.OnGetAddMenuContent = OnGetSequencerAddMenuContent;
		//ViewParams.OnGetPlaybackSpeeds = ISequencer::FOnGetPlaybackSpeeds::CreateRaw(this, &FContextualAnimViewModel::GetPlaybackSpeeds);
	}

	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.ViewParams = ViewParams;
		SequencerInitParams.RootSequence = MovieSceneSequence;
		SequencerInitParams.bEditWithinLevelEditor = false;
		SequencerInitParams.ToolkitHost = nullptr;
		SequencerInitParams.PlaybackContext.Bind(this, &FContextualAnimViewModel::GetPlaybackContext);
	}

	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked< ISequencerModule >("Sequencer");
	Sequencer = SequencerModule.CreateSequencer(SequencerInitParams);
	Sequencer->OnMovieSceneDataChanged().AddRaw(this, &FContextualAnimViewModel::SequencerDataChanged);
	Sequencer->OnGlobalTimeChanged().AddRaw(this, &FContextualAnimViewModel::SequencerTimeChanged);
	Sequencer->OnPlayEvent().AddRaw(this, &FContextualAnimViewModel::SequencerPlayEvent);
	Sequencer->OnStopEvent().AddRaw(this, &FContextualAnimViewModel::SequencerStopEvent);
	Sequencer->GetSelectionChangedSections().AddRaw(this, &FContextualAnimViewModel::OnSequencerSelectionChangedSections);
	Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stopped);
}

float FContextualAnimViewModel::GetPlaybackTime() const
{
	return Sequencer->GetGlobalTime().AsSeconds();
}

void FContextualAnimViewModel::SetActiveSection(int32 SectionIdx)
{
	check(GetSceneAsset()->Sections.IsValidIndex(SectionIdx));

	ActiveSectionIdx = SectionIdx;

	SetDefaultMode();
}

int32 FContextualAnimViewModel::GetActiveSection() const
{
	return ActiveSectionIdx;
}

void FContextualAnimViewModel::SetActiveAnimSetForSection(int32 SectionIdx, int32 AnimSetIdx)
{
	check(GetSceneAsset()->Sections.IsValidIndex(SectionIdx));
	check(GetSceneAsset()->Sections[SectionIdx].AnimSets.IsValidIndex(AnimSetIdx));

	int32& ActiveSetIdx = ActiveAnimSetMap.FindOrAdd(SectionIdx);
	ActiveSetIdx = AnimSetIdx;

	SetDefaultMode();
}

int32 FContextualAnimViewModel::GetActiveAnimSetForSection(int32 SectionIdx) const
{
	const int32* ActiveSetIdxPtr = ActiveAnimSetMap.Find(SectionIdx);
	return ActiveSetIdxPtr ? *ActiveSetIdxPtr : INDEX_NONE;
}

int32 FContextualAnimViewModel::GetDetailsSectionIdx() const
{
	return SelectionInfo.SectionIdx != INDEX_NONE ? SelectionInfo.SectionIdx : GetActiveSection();
}

int32 FContextualAnimViewModel::GetDetailsAnimSetIdx() const
{
	return SelectionInfo.AnimSetIdx != INDEX_NONE ? SelectionInfo.AnimSetIdx : GetActiveAnimSetForSection(GetDetailsSectionIdx());
}

AActor* FContextualAnimViewModel::SpawnPreviewActor(const FContextualAnimTrack& AnimTrack)
{
	AActor* PreviewActor = nullptr;

	// Prioritize override preview
	EContextualAnimActorPreviewType PreviewType = EContextualAnimActorPreviewType::None;
	USkeletalMesh* PreviewSkeletalMesh = nullptr;
	UStaticMesh* PreviewStaticMesh = nullptr;
	UClass* PreviewActorClass = nullptr;
	UClass* PreviewAnimInstanceClass = nullptr;

	IContextualAnimOverrideInterface* OverrideInterface = SceneAsset->PreviewOverrideProvider ?  Cast<IContextualAnimOverrideInterface>(SceneAsset->PreviewOverrideProvider->GetDefaultObject()) : nullptr;
	bool bPreviewOveridden = false;

	for (FContextualAnimActorPreviewData PreviewData : GetSceneAsset()->OverridePreviewData)
	{
		// Allow the PreviewOverrideProvider to override the preview data for this role
		if (OverrideInterface && OverrideInterface->GetCASAnimActorPreviewDataOverrideForRole(AnimTrack.Role, PreviewData))
		{
			bPreviewOveridden = true;
		}

		if (PreviewData.Role == AnimTrack.Role)
		{
			if (PreviewData.Type == EContextualAnimActorPreviewType::StaticMesh)
			{
				if (UStaticMesh* StaticMesh = PreviewData.PreviewStaticMesh.LoadSynchronous())
				{
					PreviewStaticMesh = StaticMesh;
					PreviewType = PreviewData.Type;
				}
			}
			else if (PreviewData.Type == EContextualAnimActorPreviewType::SkeletalMesh)
			{
				if (USkeletalMesh* SkeletalMesh = PreviewData.PreviewSkeletalMesh.LoadSynchronous())
				{
					PreviewSkeletalMesh = SkeletalMesh;
					PreviewType = PreviewData.Type;
				}

				if (UClass* AnimInstanceClass = PreviewData.PreviewAnimInstance.LoadSynchronous())
				{
					PreviewAnimInstanceClass = AnimInstanceClass;
				}
			}
			else if (PreviewData.Type == EContextualAnimActorPreviewType::Actor)
			{
				if (UClass* ActorClass = PreviewData.PreviewActorClass.LoadSynchronous())
				{
					PreviewActorClass = ActorClass;
					PreviewType = PreviewData.Type;
				}
			}
			break;
		}
	}

	// if not explicit preview mesh is defined for a role with a valid animation, try to pull the preview mesh from the animation
	if (PreviewType == EContextualAnimActorPreviewType::None && AnimTrack.Animation && AnimTrack.Animation->GetSkeleton())
	{
		PreviewSkeletalMesh = AnimTrack.Animation->GetSkeleton()->GetPreviewMesh(true);

		if (PreviewSkeletalMesh)
		{
			PreviewType = EContextualAnimActorPreviewType::SkeletalMesh;
		}
	}

	const FTransform SpawnTransform = AnimTrack.GetRootTransformAtTime(0.f);

	FActorSpawnParameters Params;
	Params.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AdjustIfPossibleButAlwaysSpawn;

	if (PreviewType != EContextualAnimActorPreviewType::None)
	{
		const FContextualAnimRoleDefinition* RoleDef = GetSceneAsset()->RolesAsset ? GetSceneAsset()->RolesAsset->FindRoleDefinitionByName(AnimTrack.Role) : nullptr;
		const bool bIsCharacter = (RoleDef && RoleDef->bIsCharacter);

		if (bIsCharacter && !bPreviewOveridden)
		{
			ACharacter* PreviewCharacter = GetWorld()->SpawnActor<ACharacter>(ACharacter::StaticClass(), SpawnTransform, Params);
			PreviewCharacter->SetFlags(RF_Transient);

			PreviewCharacter->GetCapsuleComponent()->SetCapsuleSize(RoleDef->PreviewCapsuleRadius, RoleDef->PreviewCapsuleHalfHeight);

			USkeletalMeshComponent* SkelMeshComp = PreviewCharacter->GetMesh();
			SkelMeshComp->SetRelativeTransform(RoleDef->MeshToComponent);
			SkelMeshComp->SetSkeletalMesh(PreviewSkeletalMesh);
			SkelMeshComp->SetAnimationMode(EAnimationMode::AnimationBlueprint);

			if (PreviewAnimInstanceClass)
			{
				SkelMeshComp->SetAnimInstanceClass(PreviewAnimInstanceClass);
			}
			else
			{
				SkelMeshComp->SetAnimInstanceClass(UAnimPreviewInstance::StaticClass());

				if (UAnimPreviewInstance* AnimInstance = Cast<UAnimPreviewInstance>(SkelMeshComp->GetAnimInstance()))
				{
					AnimInstance->SetAnimationAsset(AnimTrack.Animation, false, 0.f);
					AnimInstance->PlayAnim(false, 0.f);
				}
			}

			PreviewCharacter->CacheInitialMeshOffset(SkelMeshComp->GetRelativeLocation(), SkelMeshComp->GetRelativeRotation());

			if (UCharacterMovementComponent* CharacterMovementComp = PreviewCharacter->GetCharacterMovement())
			{
				CharacterMovementComp->bOrientRotationToMovement = true;
				CharacterMovementComp->bUseControllerDesiredRotation = false;
				CharacterMovementComp->RotationRate = FRotator(0.f, 540.0, 0.f);
				CharacterMovementComp->bRunPhysicsWithNoController = true;
				CharacterMovementComp->SetMovementMode(AnimTrack.MovementMode);
			}

			UMotionWarpingComponent* MotionWarpingComp = NewObject<UMotionWarpingComponent>(PreviewCharacter);
			MotionWarpingComp->RegisterComponentWithWorld(GetWorld());
			MotionWarpingComp->InitializeComponent();

			UContextualAnimSceneActorComponent* SceneActorComp = NewObject<UContextualAnimSceneActorComponent>(PreviewCharacter);
			SceneActorComp->RegisterComponentWithWorld(GetWorld());
			SceneActorComp->InitializeComponent();

			PreviewActor = PreviewCharacter;
		}
		else
		{
			if (PreviewActorClass)
			{
				PreviewActor = GetWorld()->SpawnActor<AActor>(PreviewActorClass, SpawnTransform, Params);
				PreviewActor->SetFlags(RF_Transient);

				// Allow the Override Interface to Access the PreviewActor before calling refresh, this can set any data needed to properly preview the scene with this override set
				if (OverrideInterface)
				{
					OverrideInterface->OnCASPreviewActorSpawned(AnimTrack.Role, PreviewActor);
				}
				// Try to call RefreshEditorPreview on the actor. This give game specific actors a chance to run any logic needed to be able to use them as preview actor in this editor.
				if (PreviewActor->GetClass()->ImplementsInterface(UContextualAnimActorInterface::StaticClass()))
				{
					IContextualAnimActorInterface::Execute_RefreshEditorPreview(PreviewActor);
				}

				// Validate that the spawned actor has the required UContextualAnimSceneActorComponent.
				// The user may have removed the component from the Blueprint after initially selecting a valid class.
				if (!PreviewActor->FindComponentByClass<UContextualAnimSceneActorComponent>())
				{
					ShowMissingSceneActorComponentWarning(PreviewActorClass->GetName());

					PreviewActor->Destroy();
					PreviewActor = nullptr;

					// Reset the preview data to prevent repeated warnings on subsequent refreshes
					for (FContextualAnimActorPreviewData& Data : GetSceneAsset()->OverridePreviewData)
					{
						if (Data.Role == AnimTrack.Role && Data.Type == EContextualAnimActorPreviewType::Actor)
						{
							Data.PreviewActorClass.Reset();
							break;
						}
					}
					GetSceneAsset()->MarkPackageDirty();
				}
			}
			else
			{
				PreviewActor = GetWorld()->SpawnActor<AActor>(AActor::StaticClass(), SpawnTransform, Params);
				PreviewActor->SetFlags(RF_Transient);

				if (PreviewSkeletalMesh)
				{
					UDebugSkelMeshComponent* SkelMeshComp = NewObject<UDebugSkelMeshComponent>(PreviewActor);
					SkelMeshComp->SetSkeletalMesh(PreviewSkeletalMesh);
					SkelMeshComp->SetAnimationMode(EAnimationMode::AnimationBlueprint);

					if (PreviewAnimInstanceClass)
					{
						SkelMeshComp->SetAnimInstanceClass(PreviewAnimInstanceClass);
					}
					else
					{
						SkelMeshComp->SetAnimInstanceClass(UAnimPreviewInstance::StaticClass());

						if (UAnimPreviewInstance* AnimInstance = Cast<UAnimPreviewInstance>(SkelMeshComp->GetAnimInstance()))
						{
							AnimInstance->SetAnimationAsset(AnimTrack.Animation, false, 0.f);
							AnimInstance->PlayAnim(false, 0.f);
						}
					}

					SkelMeshComp->RegisterComponentWithWorld(GetWorld());
					PreviewActor->SetRootComponent(SkelMeshComp);
				}
				else if (PreviewStaticMesh)
				{
					UStaticMeshComponent* StaticMeshComp = NewObject<UStaticMeshComponent>(PreviewActor);
					StaticMeshComp->SetStaticMesh(PreviewStaticMesh);
					StaticMeshComp->RegisterComponentWithWorld(GetWorld());
					PreviewActor->SetRootComponent(StaticMeshComp);
				}

				UContextualAnimSceneActorComponent* SceneActorComp = NewObject<UContextualAnimSceneActorComponent>(PreviewActor);
				SceneActorComp->RegisterComponentWithWorld(GetWorld());
				SceneActorComp->InitializeComponent();
			}
		}

		if (PreviewActor)
		{
			PreviewActor->Tags.Add(PreviewActorTag);
		}
	}

	return PreviewActor;
}

void FContextualAnimViewModel::ResetTimeline()
{
	for (int32 TrackIdx = MovieScene->GetTracks().Num() - 1; TrackIdx >= 0; TrackIdx--)
	{
		UMovieSceneTrack& Track = *MovieScene->GetTracks()[TrackIdx];
		MovieScene->RemoveTrack(Track);
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FContextualAnimViewModel::SetDefaultMode()
{
	ClearSelection();

	if (EditingAnimation.IsValid())
	{
		EditingAnimation->UnregisterOnNotifyChanged(this);
		EditingAnimation.Reset();
	}

	TimelineMode = ETimelineMode::Default;

	ResetTimeline();

	// Create one MovieScene track per role
	auto CreateMovieSceneTracksForRoles = [this]()
	{
		const TArray<FName> Roles = SceneAsset->GetRoles();
		for (FName Role : Roles)
		{
			UContextualAnimMovieSceneTrack* MovieSceneAnimTrack = MovieSceneSequence->GetMovieScene()->AddTrack<UContextualAnimMovieSceneTrack>();
			check(MovieSceneAnimTrack);
			MovieSceneAnimTrack->Initialize(Role);
		}
	};

	// Create MovieScene sections for all anim sets in a given section
	TArray<FName, TInlineAllocator<5>> MissingRoles;

	auto CreateMovieSceneSectionsForAnimSets = [this, &MissingRoles](const FContextualAnimSceneSection& Section, int32 SectionIdx, float StartTime, float FallbackDuration, int32 ActiveAnimSetIdx, int32 SegmentIdx = INDEX_NONE)
	{
		const FFrameRate TickResolution = MovieSceneSequence->GetMovieScene()->GetTickResolution();

		for (int32 AnimSetIdx = 0; AnimSetIdx < Section.GetNumAnimSets(); AnimSetIdx++)
		{
			const FContextualAnimSet* AnimSet = Section.GetAnimSet(AnimSetIdx);
			check(AnimSet);

			for (int32 AnimTrackIdx = 0; AnimTrackIdx < AnimSet->Tracks.Num(); AnimTrackIdx++)
			{
				const FContextualAnimTrack& AnimTrack = AnimSet->Tracks[AnimTrackIdx];

				// If the roles asset was edited, we may have tracks for roles that no longer exist in the roles asset. 
				// Collect those and warn the user
				UContextualAnimMovieSceneTrack* MovieSceneTrack = FindTrackByRole(AnimTrack.Role);
				if (MovieSceneTrack == nullptr)
				{
					MissingRoles.AddUnique(AnimTrack.Role);
					continue;
				}

				UContextualAnimMovieSceneSection* NewSection = NewObject<UContextualAnimMovieSceneSection>(MovieSceneTrack, UContextualAnimMovieSceneSection::StaticClass(), NAME_None, RF_Transactional);
				check(NewSection);

				if (AnimTrack.Animation)
				{
					const float AnimLength = FMath::Min(AnimTrack.Animation->GetPlayLength(), FallbackDuration);
					const FFrameNumber StartFrame = (StartTime * TickResolution).RoundToFrame();
					const FFrameNumber EndFrame = ((StartTime + AnimLength) * TickResolution).RoundToFrame();
					NewSection->SetRange(TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame));
					NewSection->SetRowIndex(AnimSetIdx);
					NewSection->SetIsActive(AnimSetIdx == ActiveAnimSetIdx);
				}
				else
				{
					const FFrameNumber StartFrame = (StartTime * TickResolution).RoundToFrame();
					const FFrameNumber EndFrame = ((StartTime + FallbackDuration) * TickResolution).RoundToFrame();
					NewSection->SetRange(TRange<FFrameNumber>::Inclusive(StartFrame, EndFrame));
					NewSection->SetRowIndex(AnimSetIdx);
					NewSection->SetIsActive(false);
				}

				NewSection->Initialize(SectionIdx, AnimSetIdx, AnimTrackIdx, SegmentIdx);

				MovieSceneTrack->AddSection(*NewSection);
				MovieSceneTrack->SetTrackRowDisplayName(FText::FromString(FString::Printf(TEXT("%d"), AnimSetIdx)), AnimSetIdx);
			}
		}
	};

	CreateMovieSceneTracksForRoles();

	// Multi-section: flatten all segments from the PreviewSequence into a single timeline.
	// Each segment maps to a section/anim set pair and is laid out sequentially.
	const bool bHasMultiSectionTimeline = TryBuildMultiSectionPreviewTimelineData();
	if (bHasMultiSectionTimeline)
	{
		ActiveSectionIdx = ComputedTimeline[0].SectionIdx;

		for (int32 SegIdx = 0; SegIdx < ComputedTimeline.Num(); SegIdx++)
		{
			const FPreviewTimelineSegment& Segment = ComputedTimeline[SegIdx];
			const FContextualAnimSceneSection& Section = SceneAsset->Sections[Segment.SectionIdx];
			CreateMovieSceneSectionsForAnimSets(Section, Segment.SectionIdx, Segment.GlobalStartTime, Segment.Duration, Segment.AnimSetIdx, SegIdx);
		}
	}
	// Single-section: display only the active section starting at time 0.
	else if (SceneAsset->Sections.IsValidIndex(ActiveSectionIdx))
	{
		const FContextualAnimSceneSection& ContextualAnimSection = SceneAsset->Sections[ActiveSectionIdx];

		// Calculate duration for empty tracks.
		float FallbackDuration = 0.f;
		for (int32 AnimSetIdx = 0; AnimSetIdx < ContextualAnimSection.GetNumAnimSets(); AnimSetIdx++)
		{
			if (const FContextualAnimSet* AnimSet = ContextualAnimSection.GetAnimSet(AnimSetIdx))
			{
				for (const FContextualAnimTrack& AnimTrack : AnimSet->Tracks)
				{
					if (AnimTrack.Animation)
					{
						FallbackDuration = FMath::Max(FallbackDuration, AnimTrack.Animation->GetPlayLength());
					}
				}
			}
		}
		FallbackDuration = FMath::Max(1.f, FallbackDuration);

		const int32 ActiveAnimSetIdx = ActiveAnimSetMap.FindOrAdd(ActiveSectionIdx);
		CreateMovieSceneSectionsForAnimSets(ContextualAnimSection, ActiveSectionIdx, 0.f, FallbackDuration, ActiveAnimSetIdx);
	}

	if (MissingRoles.Num() > 0)
	{
		FString RoleList;
		for (const FName& Role : MissingRoles)
		{
			if (!RoleList.IsEmpty())
			{
				RoleList += TEXT(", ");
			}
			RoleList += Role.ToString();
		}

		FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
			LOCTEXT("MissingRolesWarning",
				"Scene Asset '{0}' has tracks for roles that no longer exist in Role Asset '{1}':\n\n{2}\n\n"
				"These tracks will be skipped. Use 'Update Roles' in the Details panel to clean up stale roles."),
			FText::FromString(GetNameSafe(SceneAsset)),
			FText::FromString(GetNameSafe(SceneAsset->GetRolesAsset())),
			FText::FromString(RoleList)));
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);

	RefreshPreviewScene();
}

void FContextualAnimViewModel::SetNotifiesMode(const FContextualAnimTrack& AnimTrack)
{
	ResetTimeline();

	TimelineMode = ETimelineMode::Notifies;

	if(EditingAnimation.IsValid())
	{
		EditingAnimation->UnregisterOnNotifyChanged(this);
	}

	EditingAnimation = AnimTrack.Animation;

	// Add Animation Track
	{
		UContextualAnimMovieSceneTrack* MovieSceneAnimTrack = MovieSceneSequence->GetMovieScene()->AddTrack<UContextualAnimMovieSceneTrack>();
		check(MovieSceneAnimTrack);

		MovieSceneAnimTrack->SetDisplayName(FText::FromString(GetNameSafe(EditingAnimation.Get())));

		UContextualAnimMovieSceneSection* NewSection = NewObject<UContextualAnimMovieSceneSection>(MovieSceneAnimTrack, UContextualAnimMovieSceneSection::StaticClass(), NAME_None, RF_Transactional);
		check(NewSection);

		NewSection->Initialize(AnimTrack.SectionIdx, AnimTrack.AnimSetIdx, AnimTrack.AnimTrackIdx);

		FFrameRate TickResolution = MovieSceneSequence->GetMovieScene()->GetTickResolution();
		FFrameNumber StartFrame(0);
		FFrameNumber EndFrame = (EditingAnimation->GetPlayLength() * TickResolution).RoundToFrame();
		NewSection->SetRange(TRange<FFrameNumber>::Exclusive(StartFrame, EndFrame));

		MovieSceneAnimTrack->AddSection(*NewSection);
	}

	// Add Notify Tracks
	{
		for (const FAnimNotifyTrack& NotifyTrack : EditingAnimation->AnimNotifyTracks)
		{
			UContextualAnimMovieSceneNotifyTrack* Track = MovieSceneSequence->GetMovieScene()->AddTrack<UContextualAnimMovieSceneNotifyTrack>();
			check(Track);

			Track->Initialize(*EditingAnimation, NotifyTrack);
		}

		// Listen for when the notifies in the animation changes, so we can refresh the notify sections here
		AnimTrack.Animation->RegisterOnNotifyChanged(UAnimSequenceBase::FOnNotifyChanged::CreateSP(this, &FContextualAnimViewModel::OnAnimNotifyChanged, EditingAnimation.Get()));
	}

	Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::MovieSceneStructureItemsChanged);
}

void FContextualAnimViewModel::RefreshPreviewScene()
{
	SceneBindings.Reset();

	// Remove preview actors from the preview scene
	// @TODO: Investigate why GetActor from each binding in SceneBindings returns null after a Redo op
	for (TActorIterator<AActor> It(GetWorld()); It; ++It)
	{
		AActor* Actor = *It;
		if (Actor && Actor->ActorHasTag(PreviewActorTag))
		{
			Actor->Destroy();
		}
	}

	if (!SceneAsset->HasValidData())
	{
		return;
	}

	if (!SceneAsset->Sections.IsValidIndex(ActiveSectionIdx))
	{
		return;
	}

	const FContextualAnimSceneSection& ContextualAnimSection = SceneAsset->Sections[ActiveSectionIdx];
	const int32 ActiveAnimSetIdx = ActiveAnimSetMap.FindOrAdd(ActiveSectionIdx);
	if (!ContextualAnimSection.AnimSets.IsValidIndex(ActiveAnimSetIdx))
	{
		return;
	}

	SceneBindings = FContextualAnimSceneBindings(*SceneAsset);

	const FContextualAnimSet& AnimSet = ContextualAnimSection.AnimSets[ActiveAnimSetIdx];
	for (const FContextualAnimTrack& AnimTrack : AnimSet.Tracks)
	{
		if (AActor* PreviewActor = SpawnPreviewActor(AnimTrack))
		{
			SceneBindings.BindActorToRole(*PreviewActor, AnimTrack.Role, ActiveSectionIdx, ActiveAnimSetIdx);
		}
	}

	if (SceneAsset->PreviewOverrideProvider)
	{
		SceneBindings.SetOverrideProvider(TScriptInterface<IContextualAnimOverrideInterface>(SceneAsset->PreviewOverrideProvider->GetDefaultObject()));
	}

	// Pause all anims and disable auto blend out
	const FContextualAnimSceneBinding* Leader = SceneBindings.GetSyncLeader();
	if (Leader && Leader->GetSceneActorComponent()->StartContextualAnimScene(SceneBindings, {}))
	{
		for (const FContextualAnimSceneBinding& Binding : SceneBindings)
		{
			if (FAnimMontageInstance* MontageInstance = Binding.GetAnimMontageInstance())
			{
				MontageInstance->Pause();
				MontageInstance->bEnableAutoBlendOut = false;
			}
		}
	}
}

void FContextualAnimViewModel::AddNewAnimSet(const FContextualAnimNewAnimSetParams& Params)
{
	FContextualAnimSet AnimSet;
	for (const FContextualAnimNewAnimSetData& Data : Params.Data)
	{
		FContextualAnimTrack AnimTrack;
		AnimTrack.Role = Data.RoleName;
		AnimTrack.Animation = Data.Animation;
		AnimTrack.MovementMode = Data.MovementMode;
		AnimTrack.CustomMovementMode = Data.CustomMovementMode;
		AnimTrack.bOptional = Data.bOptional;
		AnimSet.Tracks.Add(AnimTrack);
		AnimSet.RandomWeight = Params.RandomWeight;

		// Update preview mesh if necessary
		if (Data.Animation && AnimTrack.Animation->GetSkeleton())
		{
			const FName Role = Data.RoleName;
			if (!SceneAsset->OverridePreviewData.ContainsByPredicate([Role](const FContextualAnimActorPreviewData& Item) { return Item.Role == Role; }))
			{
				if (USkeletalMesh* SkelMesh = AnimTrack.Animation->GetSkeleton()->GetPreviewMesh(true))
				{
					FContextualAnimActorPreviewData NewPreviewData;
					NewPreviewData.Role = Role;
					NewPreviewData.Type = EContextualAnimActorPreviewType::SkeletalMesh;
					NewPreviewData.PreviewSkeletalMesh = SkelMesh;
					SceneAsset->OverridePreviewData.Add(NewPreviewData);
				}
			}
		}
	}

	int32 SectionIdx = INDEX_NONE;
	int32 AnimSetIdx = INDEX_NONE;
	for(int32 Idx = 0; Idx < SceneAsset->Sections.Num(); Idx++)
	{
		FContextualAnimSceneSection& Section = SceneAsset->Sections[Idx];
		if(Section.Name == Params.SectionName)
		{
			AnimSetIdx = Section.AnimSets.Add(AnimSet);
			SectionIdx = Idx;
			break;
		}
	}
	
	if(SectionIdx == INDEX_NONE)
	{
		FContextualAnimSceneSection NewSection;
		NewSection.Name = Params.SectionName;
		AnimSetIdx = NewSection.AnimSets.Add(AnimSet);
		SectionIdx = SceneAsset->Sections.Add(NewSection);
	}

	check(SectionIdx != INDEX_NONE);
	check(AnimSetIdx != INDEX_NONE);

	SceneAsset->PrecomputeData();

	SceneAsset->MarkPackageDirty();

	// Set active AnimSet and refresh sequencer panel
	SetActiveAnimSetForSection(SectionIdx, AnimSetIdx);
}

void FContextualAnimViewModel::RemoveSection(int32 SectionIdx)
{
 	check(SceneAsset->Sections.IsValidIndex(SectionIdx));

	FScopedTransaction Transaction(LOCTEXT("RemoveSection", "Remove Section"));
	SceneAsset->Modify();

 	SceneAsset->Sections.RemoveAt(SectionIdx);

	if (SceneAsset->GetNumSections() > 0)
	{
		SetActiveAnimSetForSection(0, 0);
	}
	else
	{
		ActiveSectionIdx = INDEX_NONE;
		ActiveAnimSetMap.Reset();
	}

	SetDefaultMode();
}

void FContextualAnimViewModel::RemoveAnimSet(int32 SectionIdx, int32 AnimSetIdx)
{
	check(SceneAsset->Sections.IsValidIndex(SectionIdx));
	check(SceneAsset->Sections[SectionIdx].AnimSets.IsValidIndex(AnimSetIdx));

	FScopedTransaction Transaction(LOCTEXT("RemoveAnimSet", "Remove Anim Set"));
	SceneAsset->Modify();

	SceneAsset->Sections[SectionIdx].AnimSets.RemoveAt(AnimSetIdx);

	// If there are other sets in the current section, switch to the first one
	if (SceneAsset->Sections[SectionIdx].GetNumAnimSets() > 0)
	{
		SetActiveAnimSetForSection(SectionIdx, 0);

		SetDefaultMode();
	}
	else // Otherwise, remove the section too, there is no point in having a section without sets
	{
		RemoveSection(SectionIdx);
	}
}

void FContextualAnimViewModel::AddNewIKTarget(const UContextualAnimNewIKTargetParams& Params)
{
	check(SceneAsset->Sections.IsValidIndex(Params.SectionIdx));

	// Add IK Target definition to the scene asset
	FContextualAnimIKTargetDefinition IKTargetDef;
	IKTargetDef.GoalName = Params.GoalName;
	IKTargetDef.BoneName = Params.SourceBone.BoneName;
	IKTargetDef.Provider = Params.Provider;
	IKTargetDef.TargetRoleName = Params.TargetRole;
	IKTargetDef.TargetBoneName = Params.TargetBone.BoneName;

	FName Role = Params.SourceRole;
	if (FContextualAnimIKTargetDefContainer* ContainerPtr = SceneAsset->IKTargetParams.IKTargetDefsForEachRole.FindByPredicate([Role](const FContextualAnimIKTargetDefContainer& Item) { return Item.Role == Role; }))
	{
		ContainerPtr->IKTargetDefs.AddUnique(IKTargetDef);
	}
	else
	{
		FContextualAnimIKTargetDefContainer Container;
		Container.Role = Role;
		Container.IKTargetDefs.Add(IKTargetDef);
		SceneAsset->IKTargetParams.IKTargetDefsForEachRole.Add(Container);
	}

	SceneAsset->PrecomputeData();

	SceneAsset->MarkPackageDirty();
}

void FContextualAnimViewModel::StartSimulateMode()
{
	Sequencer->Pause();
	Sequencer->SetGlobalTime(0);

	MovieScene->SetReadOnly(true);

	SimulateModeState = ESimulateModeState::Paused;

	// Stop montage instance used during normal mode to preview the animations
	for (const FContextualAnimSceneBinding& Binding : SceneBindings)
	{
		if (UAnimInstance* AnimInstance = Binding.GetAnimInstance())
		{
			AnimInstance->StopAllMontages(0);
		}
	}
}

void FContextualAnimViewModel::StopSimulateMode()
{
	SimulateModeState = ESimulateModeState::Inactive;
	CurrentSegmentIdx = INDEX_NONE;

	Sequencer->Pause();
	Sequencer->SetGlobalTime(0);

	MovieScene->SetReadOnly(false);

	for (auto& Binding : SceneBindings)
	{
		if (UMotionWarpingComponent* MotionWarpComp = Binding.GetActor()->FindComponentByClass<UMotionWarpingComponent>())
		{
			const FContextualAnimSceneSection* Section = SceneAsset->GetSection(Binding.GetSectionIdx());
			check(Section);

			for (const FContextualAnimWarpPointDefinition& WarpPointDef : Section->GetWarpPointDefinitions())
			{
				MotionWarpComp->RemoveWarpTarget(WarpPointDef.WarpTargetName);
			}
		}
	}

	SetDefaultMode();
}

void FContextualAnimViewModel::ToggleSimulateMode()
{
	if (SimulateModeState == ESimulateModeState::Inactive)
	{
		StartSimulateMode();
	}
	else
	{
		StopSimulateMode();
	}

	ClearSelection();
};

void FContextualAnimViewModel::StartSimulation()
{
	// For multi-section mode, start with the first segment's section
	const int32 StartSectionIdx = HasMultiSectionPreviewTimelineData() ? ComputedTimeline[0].SectionIdx : ActiveSectionIdx;

	TMap<FName, FContextualAnimSceneBindingContext> RoleToActorMap;
	for (const FContextualAnimSceneBinding& Binding : SceneBindings)
	{
		RoleToActorMap.Add(SceneBindings.GetRoleFromBinding(Binding), Binding.GetContext());
	}

	bool bStarted = false;
	if (FContextualAnimSceneBindings::TryCreateBindings(*GetSceneAsset(), StartSectionIdx, RoleToActorMap, SceneBindings))
	{
		const FContextualAnimSceneBinding* Leader = SceneBindings.GetSyncLeader();
		if (Leader && Leader->GetSceneActorComponent()->StartContextualAnimScene(SceneBindings, {}))
		{
			SimulateModeState = ESimulateModeState::Playing;
			bStarted = true;

			if (HasMultiSectionPreviewTimelineData())
			{
				CurrentSegmentIdx = 0;
			}
		}
	}

	// @TODO: Temp solution for when the interaction fails to start during Simulation Mode
	// TryCreateBindings should bubble up the error so we can show what went wrong to the user right here
	if (!bStarted)
	{
		StopSimulateMode();

		FMessageDialog::Open(EAppMsgType::Ok,
			LOCTEXT("ContextualAnimSceneStartFailed",
				"Contextual Anim Scene failed to start. Check the Output Log with 'LogContextualAnim' set to All for more details."));
	}
}

void FContextualAnimViewModel::TickSimulateMode(float DeltaTime)
{
	if (SimulateModeState != ESimulateModeState::Playing || !HasMultiSectionPreviewTimelineData())
	{
		return;
	}

	if (!ComputedTimeline.IsValidIndex(CurrentSegmentIdx))
	{
		return;
	}

	const FContextualAnimSceneBinding* Leader = SceneBindings.GetSyncLeader();
	if (!Leader)
	{
		return;
	}

	FAnimMontageInstance* MontageInstance = Leader->GetAnimMontageInstance();
	if (!MontageInstance || !MontageInstance->Montage)
	{
		// Montage was removed (e.g. scene ended via LeaveScene on the last segment)
		StopSimulateMode();
		return;
	}

	const float Position = MontageInstance->GetPosition();
	const float Length = MontageInstance->Montage->GetPlayLength();
	const float BlendOutTime = MontageInstance->Montage->BlendOut.GetBlendTime();

	// Trigger transition at the point where the montage starts blending out
	const float TransitionPoint = Length - BlendOutTime;

	if (Position >= TransitionPoint - KINDA_SMALL_NUMBER)
	{
		const int32 NextSegmentIdx = CurrentSegmentIdx + 1;
		if (NextSegmentIdx < ComputedTimeline.Num())
		{
			const FPreviewTimelineSegment& NextSegment = ComputedTimeline[NextSegmentIdx];
			const FContextualAnimSceneSection* Section = SceneAsset->GetSection(NextSegment.SectionIdx);
			check(Section);

			if (UContextualAnimSceneActorComponent* LeaderComp = Leader->GetSceneActorComponent())
			{
				if (!LeaderComp->TransitionContextualAnimSceneToSpecificSet(Section->GetName(), NextSegment.AnimSetIdx))
				{
					StopSimulateMode();

					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
						LOCTEXT("TransitionFailedDuringSimulate",
							"Failed to transition to section '{0}' (AnimSet {1}) during simulation. "
							"Simulation has been stopped. Check the Output Log for more details."),
						FText::FromName(Section->GetName()),
						FText::AsNumber(NextSegment.AnimSetIdx)));
					return;
				}

				// Keep ViewModel's SceneBindings in sync with the component
				SceneBindings = LeaderComp->GetBindings();
			}

			CurrentSegmentIdx = NextSegmentIdx;
		}
		// Last segment: let the montage end naturally (OnMontageBlendingOut -> LeaveScene)
	}
}

bool FContextualAnimViewModel::TryBuildMultiSectionPreviewTimelineData()
{
	ComputedTimeline.Reset();
	TotalSequenceDuration = 0.f;
	CurrentSegmentIdx = INDEX_NONE;

	if (!SceneAsset || SceneAsset->MultiSectionPreviewSequence.Num() == 0)
	{
		return false;
	}

	for (const FContextualAnimPreviewSequenceEntry& Entry : SceneAsset->MultiSectionPreviewSequence)
	{
		const int32 SectionIdx = SceneAsset->GetSectionIndex(Entry.SectionName);
		if (SectionIdx == INDEX_NONE)
		{
			UE_LOGF(LogContextualAnim, Warning, "TryBuildMultiSectionPreviewTimelineData: Section '%ls' not found in asset, skipping.", *Entry.SectionName.ToString());
			continue;
		}

		const FContextualAnimSceneSection* Section = SceneAsset->GetSection(SectionIdx);
		check(Section);

		// Determine which anim set to use
		int32 ResolvedAnimSetIdx = Entry.AnimSetIdx;
		if (ResolvedAnimSetIdx == INDEX_NONE)
		{
			// Use the editor's active anim set for this section
			const int32* ActiveSetIdxPtr = ActiveAnimSetMap.Find(SectionIdx);
			ResolvedAnimSetIdx = ActiveSetIdxPtr ? *ActiveSetIdxPtr : 0;
		}

		if (!Section->GetAnimSet(ResolvedAnimSetIdx))
		{
			UE_LOGF(LogContextualAnim, Warning, "TryBuildMultiSectionPreviewTimelineData: AnimSet %d not valid for section '%ls', skipping.", ResolvedAnimSetIdx, *Entry.SectionName.ToString());
			continue;
		}

		// Compute duration as max animation length across all tracks in the resolved anim set
		const FContextualAnimSet* AnimSet = Section->GetAnimSet(ResolvedAnimSetIdx);
		float SegmentDuration = 0.f;
		for (const FContextualAnimTrack& Track : AnimSet->Tracks)
		{
			if (Track.Animation)
			{
				SegmentDuration = FMath::Max(SegmentDuration, Track.Animation->GetPlayLength());
			}
		}
		SegmentDuration = FMath::Max(SegmentDuration, KINDA_SMALL_NUMBER);

		// Expand loops
		for (int32 Loop = 0; Loop < Entry.LoopCount; Loop++)
		{
			FPreviewTimelineSegment Segment;
			Segment.SectionIdx = SectionIdx;
			Segment.AnimSetIdx = ResolvedAnimSetIdx;
			Segment.LoopIteration = Loop;
			Segment.GlobalStartTime = TotalSequenceDuration;
			Segment.Duration = SegmentDuration;

			ComputedTimeline.Add(Segment);
			TotalSequenceDuration += SegmentDuration;
		}
	}

	return ComputedTimeline.Num() > 0;
}

bool FContextualAnimViewModel::ResolveGlobalTime(float GlobalTime, int32& OutSectionIdx, int32& OutAnimSetIdx, float& OutLocalTime, int32& OutSegmentIdx) const
{
	if (ComputedTimeline.Num() == 0)
	{
		return false;
	}

	// Clamp to valid range
	GlobalTime = FMath::Clamp(GlobalTime, 0.f, TotalSequenceDuration - KINDA_SMALL_NUMBER);

	// Find which segment contains this global time
	for (int32 Idx = 0; Idx < ComputedTimeline.Num(); Idx++)
	{
		const FPreviewTimelineSegment& Segment = ComputedTimeline[Idx];
		const float SegmentEndTime = Segment.GlobalStartTime + Segment.Duration;

		if (GlobalTime < SegmentEndTime || Idx == ComputedTimeline.Num() - 1)
		{
			OutSectionIdx = Segment.SectionIdx;
			OutAnimSetIdx = Segment.AnimSetIdx;
			OutLocalTime = FMath::Clamp(GlobalTime - Segment.GlobalStartTime, 0.f, Segment.Duration - KINDA_SMALL_NUMBER);
			OutSegmentIdx = Idx;
			return true;
		}
	}

	return false;
}

void FContextualAnimViewModel::TransitionPreviewToSegment(int32 SegmentIdx)
{
	check(ComputedTimeline.IsValidIndex(SegmentIdx));

	const FPreviewTimelineSegment& Segment = ComputedTimeline[SegmentIdx];
	const FContextualAnimSceneSection* Section = SceneAsset->GetSection(Segment.SectionIdx);
	check(Section);

	// Use the runtime transition mechanism to swap montages
	const FContextualAnimSceneBinding* Leader = SceneBindings.GetSyncLeader();
	if (Leader)
	{
		if (UContextualAnimSceneActorComponent* LeaderComp = Leader->GetSceneActorComponent())
		{
			if (!LeaderComp->TransitionContextualAnimSceneToSpecificSet(Section->GetName(), Segment.AnimSetIdx))
			{
				FMessageDialog::Open(EAppMsgType::Ok, FText::Format(
					LOCTEXT("TransitionFailedDuringScrub",
						"Failed to transition to section '{0}' (AnimSet {1}) while scrubbing. "
						"Check the Output Log for more details."),
					FText::FromName(Section->GetName()),
					FText::AsNumber(Segment.AnimSetIdx)));
				return;
			}

			// Sync our SceneBindings from the component's updated copy
			// so that subsequent reads (UpdatePreviewActorTransform, etc.)
			// use the correct SectionIdx / AnimSetIdx / AnimTrack data.
			SceneBindings = LeaderComp->GetBindings();
		}
	}

	// Pause all montages and disable auto blend out (same as RefreshPreviewScene does after starting a scene)
	for (const FContextualAnimSceneBinding& Binding : SceneBindings)
	{
		if (FAnimMontageInstance* MontageInstance = Binding.GetAnimMontageInstance())
		{
			MontageInstance->Pause();
			MontageInstance->bEnableAutoBlendOut = false;
		}
	}

	CurrentSegmentIdx = SegmentIdx;
}

UWorld* FContextualAnimViewModel::GetWorld() const
{
	check(PreviewScenePtr.IsValid());
	return PreviewScenePtr.Pin()->GetWorld();
}

UObject* FContextualAnimViewModel::GetPlaybackContext() const
{
	return GetWorld();
}

void FContextualAnimViewModel::UpdatePreviewActorTransform(const FContextualAnimSceneBinding& Binding, float Time)
{
	if (AActor* PreviewActor = Binding.GetActor())
	{
		const FContextualAnimTrack& AnimTrack = SceneBindings.GetAnimTrackFromBinding(Binding);

		// Skip position update for animations with ForceRootLock since they have no meaningful root motion data. 
		// Preserves the actor's current position from the previous section.
		if (HasForceRootLock(AnimTrack.Animation.Get(), Time))
		{
			return;
		}

		FTransform Transform = AnimTrack.GetRootTransformAtTime(Time);

		// Special case for Character
		if (ACharacter* PreviewCharacter = Cast<ACharacter>(PreviewActor))
		{
			if (UCharacterMovementComponent* MovementComp = Binding.GetActor()->FindComponentByClass<UCharacterMovementComponent>())
			{
				MovementComp->StopMovementImmediately();
			}

			const float MIN_FLOOR_DIST = 1.9f; //from CharacterMovementComp, including this offset to avoid jittering in walking mode
			const float CapsuleHalfHeight = PreviewCharacter->GetCapsuleComponent()->GetScaledCapsuleHalfHeight();
			Transform.SetLocation(Transform.GetLocation() + (PreviewCharacter->GetActorQuat().GetUpVector() * (CapsuleHalfHeight + MIN_FLOOR_DIST)));

			Transform.SetRotation(PreviewCharacter->GetBaseRotationOffset().Inverse() * Transform.GetRotation());
		}

		PreviewActor->SetActorLocationAndRotation(Transform.GetLocation(), Transform.GetRotation());
	}
}

UContextualAnimMovieSceneTrack* FContextualAnimViewModel::FindTrackByRole(const FName& Role) const
{
	const TArray<UMovieSceneTrack*>& Tracks = MovieScene->GetTracks();
	for (UMovieSceneTrack* Track : Tracks)
	{
		UContextualAnimMovieSceneTrack* ContextualAnimTrack = Cast<UContextualAnimMovieSceneTrack>(Track);
		check(ContextualAnimTrack);

		if (ContextualAnimTrack->GetRole() == Role)
		{
			return ContextualAnimTrack;
		}
	}

	return nullptr;
}

void FContextualAnimViewModel::SequencerPlayEvent()
{
	if (SimulateModeState == ESimulateModeState::Paused)
	{
		StartSimulation();
	}
}

void FContextualAnimViewModel::SequencerStopEvent()
{
	if (SimulateModeState != ESimulateModeState::Inactive)
	{
		StopSimulateMode();
	}
}

void FContextualAnimViewModel::SequencerTimeChanged()
{
	if (!IsSimulateModeInactive())
	{
		PreviousSequencerStatus = EMovieScenePlayerStatus::Stopped;
		PreviousSequencerTime = 0.f;
		return;
	}

	EMovieScenePlayerStatus::Type CurrentStatus = Sequencer->GetPlaybackStatus();
	const float CurrentSequencerTime = Sequencer->GetGlobalTime().AsSeconds();

	if (HasMultiSectionPreviewTimelineData())
	{
		// Multi-section scrubbing: resolve global time to section-local coordinates
		int32 SectionIdx, AnimSetIdx, SegmentIdx;
		float LocalTime;

		if (ResolveGlobalTime(CurrentSequencerTime, SectionIdx, AnimSetIdx, LocalTime, SegmentIdx))
		{
			// Transition to new segment if needed (handles both forward and backward scrubbing)
			if (SegmentIdx != CurrentSegmentIdx)
			{
				TransitionPreviewToSegment(SegmentIdx);
			}

			// Set montage positions from resolved local time
			for (const FContextualAnimSceneBinding& Binding : SceneBindings)
			{
				if (FAnimMontageInstance* MontageInstance = Binding.GetAnimMontageInstance())
				{
					const float AnimPlayLength = MontageInstance->Montage->GetPlayLength();
					const float ClampedTime = FMath::Clamp(LocalTime, 0.f, AnimPlayLength - KINDA_SMALL_NUMBER);
					MontageInstance->SetPosition(ClampedTime);
					UpdatePreviewActorTransform(Binding, ClampedTime);
				}
			}
		}
	}
	else
	{
		// Single-section scrubbing
		for (const FContextualAnimSceneBinding& Binding : SceneBindings)
		{
			if (FAnimMontageInstance* MontageInstance = Binding.GetAnimMontageInstance())
			{
				const float AnimPlayLength = MontageInstance->Montage->GetPlayLength();
				const float CurrentTime = FMath::Clamp(CurrentSequencerTime, 0.f, AnimPlayLength - KINDA_SMALL_NUMBER);
				MontageInstance->SetPosition(CurrentTime);
				UpdatePreviewActorTransform(Binding, CurrentTime);
			}
		}
	}

	PreviousSequencerStatus = CurrentStatus;
	PreviousSequencerTime = CurrentSequencerTime;
}

void FContextualAnimViewModel::SequencerDataChanged(EMovieSceneDataChangeType DataChangeType)
{
	UE_LOGF(LogContextualAnim, Log, "FContextualAnimViewModel::OnMovieSceneDataChanged DataChangeType: %d", (int32)DataChangeType);

	// Stop simulate mode if the user clicks the Unlock button on the Sequencer Panel while Simulate mode is active
	// This is a temp solution, in the future we will hide that button along with the other Sequencer related buttons
	if (DataChangeType == EMovieSceneDataChangeType::Unknown && SimulateModeState != ESimulateModeState::Inactive && Sequencer->IsReadOnly() == false)
	{
		ToggleSimulateMode();
		return;
	}

	if(DataChangeType == EMovieSceneDataChangeType::TrackValueChanged)
	{
		// Update IK AnimNotify's bEnable flag based on the Active state of the section
		// @TODO: Temp brute-force approach until having a way to override FMovieSceneSection::SetIsActive or something similar

		// @TODO: Commented out for now until we add the new behavior where the user needs to double-click on the animation to edit the notifies
		/*for (const auto& Binding : SceneInstance->GetBindings())
		{
			TArray<UMovieSceneTrack*> Tracks = MovieSceneSequence->GetMovieScene()->FindTracks(UContextualAnimMovieSceneNotifyTrack::StaticClass(), Binding.Guid);
			for(UMovieSceneTrack* Track : Tracks)
			{
				const TArray<UMovieSceneSection*>& Sections = Track->GetAllSections();
				for(UMovieSceneSection* Section : Sections)
				{
					if(UContextualAnimMovieSceneNotifySection* NotifySection = Cast<UContextualAnimMovieSceneNotifySection>(Section))
					{
						if(UAnimNotifyState_IKWindow* IKNotify = Cast<UAnimNotifyState_IKWindow>(NotifySection->GetAnimNotifyState()))
						{
							if(IKNotify->bEnable != NotifySection->IsActive())
							{
								IKNotify->bEnable = NotifySection->IsActive();
								IKNotify->MarkPackageDirty();
							}
						}
					}
				}
			}
		}*/
	}
}

void FContextualAnimViewModel::OnAnimNotifyChanged(UAnimSequenceBase* Animation)
{
	// Do not refresh sequencer tracks if the change to the notifies came from us
	if (bUpdatingAnimationFromSequencer)
	{
		return;
	}

	UE_LOGF(LogContextualAnim, Log, "FContextualAnimViewModel::OnAnimNotifyChanged Anim: %ls. Refreshing Sequencer Tracks", *GetNameSafe(Animation));

	// Refresh notifies view if active
	if(TimelineMode == ETimelineMode::Notifies && EditingAnimation.Get() == Animation)
	{
		if(const FContextualAnimTrack* AnimTrack = GetSceneAsset()->FindAnimTrackByAnimation(Animation))
		{
			SetNotifiesMode(*AnimTrack);
		}
	}
}

void FContextualAnimViewModel::AnimationModified(UAnimSequenceBase& Animation)
{
	TGuardValue<bool> UpdateGuard(bUpdatingAnimationFromSequencer, true);

	Animation.RefreshCacheData();
	Animation.PostEditChange();
	Animation.MarkPackageDirty();
}

void FContextualAnimViewModel::RefreshDetailsView()
{
	DetailsView->ForceRefresh();
}

void FContextualAnimViewModel::OnSequencerSelectionChangedSections(TArray<UMovieSceneSection*> Sections)
{
	UE_LOGF(LogContextualAnim, Log, "OnSequencerSelectionChangedSections Sections.Num: %d", Sections.Num());

	if (Sections.Num() > 0)
	{
		if (UContextualAnimMovieSceneSection* Section = Cast<UContextualAnimMovieSceneSection>(Sections[0]))
		{
			UpdateSelection(Section->GetSectionIdx(), Section->GetAnimSetIdx(), Section->GetAnimTrackIdx());
		}
	}
}

void FContextualAnimViewModel::UpdateSelection(const AActor* SelectedActor)
{
	const FContextualAnimSceneBinding* Binding = SceneBindings.FindBindingByActor(SelectedActor);
	if (Binding)
	{
		UpdateSelection(SceneBindings.GetRoleFromBinding(*Binding));
	}
	else
	{
		ClearSelection();
	}
}

void FContextualAnimViewModel::UpdateSelection(FName Role, int32 CriterionIdx, int32 CriterionDataIdx)
{
	Sequencer->EmptySelection();

	SelectionInfo.Role = Role;

	// In multi-section mode, use the current segment's section/animset (based on scrub position)
	if (HasMultiSectionPreviewTimelineData() && ComputedTimeline.IsValidIndex(CurrentSegmentIdx))
	{
		const FPreviewTimelineSegment& Segment = ComputedTimeline[CurrentSegmentIdx];
		SelectionInfo.SectionIdx = Segment.SectionIdx;
		SelectionInfo.AnimSetIdx = Segment.AnimSetIdx;
	}
	else
	{
		SelectionInfo.SectionIdx = ActiveSectionIdx;
		SelectionInfo.AnimSetIdx = ActiveAnimSetMap.FindRef(ActiveSectionIdx);
	}

	SelectionInfo.Criterion.Key = CriterionIdx;
	SelectionInfo.Criterion.Value = CriterionDataIdx;

	RefreshDetailsView();
}

void FContextualAnimViewModel::UpdateSelection(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx)
{
	if (const FContextualAnimTrack* AnimTrack = SceneAsset->GetAnimTrack(SectionIdx, AnimSetIdx, AnimTrackIdx))
	{
		Sequencer->EmptySelection();

		SelectionInfo.Role = AnimTrack->Role;
		SelectionInfo.SectionIdx = SectionIdx;
		SelectionInfo.AnimSetIdx = AnimSetIdx;
		SelectionInfo.Criterion.Key = INDEX_NONE;
		SelectionInfo.Criterion.Value = INDEX_NONE;

		RefreshDetailsView();
	}
}

void FContextualAnimViewModel::ClearSelection()
{
	SelectionInfo.Reset();

	RefreshDetailsView();
}

FContextualAnimSceneBinding* FContextualAnimViewModel::GetSelectedBinding() const
{
	return const_cast<FContextualAnimSceneBinding*>(SceneBindings.FindBindingByRole(SelectionInfo.Role));
}

AActor* FContextualAnimViewModel::GetSelectedActor() const
{
	const FContextualAnimSceneBinding* Binding = GetSelectedBinding();
	return Binding ? Binding->GetActor() : nullptr;
}

FContextualAnimTrack* FContextualAnimViewModel::GetSelectedAnimTrack() const
{
	const FContextualAnimTrack* AnimTrack = SceneAsset->GetAnimTrack(SelectionInfo.SectionIdx, SelectionInfo.AnimSetIdx, SelectionInfo.Role);
	return AnimTrack ? const_cast<FContextualAnimTrack*>(AnimTrack) : nullptr;
}

UContextualAnimSelectionCriterion* FContextualAnimViewModel::GetSelectedSelectionCriterion() const
{
	if (SelectionInfo.Criterion.Key != INDEX_NONE && SelectionInfo.Criterion.Value != INDEX_NONE)
	{
		FContextualAnimTrack* AnimTrack = GetSelectedAnimTrack();
		if (AnimTrack && AnimTrack->SelectionCriteria.IsValidIndex(SelectionInfo.Criterion.Key))
		{
			return AnimTrack->SelectionCriteria[SelectionInfo.Criterion.Key];
		}
	}

	return nullptr;
}

FText FContextualAnimViewModel::GetSelectionDebugText() const
{
	AActor* SelectedActor = GetSelectedActor();

	FString AttachmentText = "None";
	if (SelectedActor)
	{
		if (const FContextualAnimAttachmentParams* Params = SceneAsset->GetAttachmentParamsForRole(SelectionInfo.Role))
		{
			if (const FContextualAnimSceneBinding* Primary = SceneBindings.GetPrimaryBinding())
			{
				if (const UMeshComponent* MeshComp = UContextualAnimUtilities::TryGetMeshComponentWithSocket(Primary->GetActor(), Params->SocketName))
				{
					const FTransform SocketTransform = MeshComp->GetSocketTransform(Params->SocketName, ERelativeTransformSpace::RTS_World);
					const FTransform RelativeTransform = SelectedActor->GetTransform().GetRelativeTransform(SocketTransform);
					const FVector Loc = RelativeTransform.GetLocation();
					const FRotator Rot = RelativeTransform.Rotator();
					AttachmentText = FString::Printf(TEXT("\n    Location: X=%.3f, Y=%.3f, Z=%.3f\n    Rotation: P=%.3f, Y=%.3f, R=%.3f"),
						Loc.X, Loc.Y, Loc.Z, Rot.Pitch, Rot.Yaw, Rot.Roll);
				}
			}
		}
	}
	FString ActiveSectionText = TEXT("None");
	const int32 SectionIdx = GetActiveSection();
	if (const FContextualAnimSceneSection* Section = SceneAsset->GetSection(SectionIdx))
	{
		ActiveSectionText = FString::Printf(TEXT("[%d] %s"), SectionIdx, *Section->GetName().ToString());
	}

	return FText::FromString(FString::Printf(TEXT("Active Section: %s\n\nSelection Info:\n Role: %s \n Attachment Relative Transform: %s \n Criterion: %d (%d)"),
		*ActiveSectionText, *SelectionInfo.Role.ToString(), *AttachmentText, SelectionInfo.Criterion.Key, SelectionInfo.Criterion.Value));
}

bool FContextualAnimViewModel::ProcessInputDelta(FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (IsSimulateModePaused())
	{
		// In Simulate Mode (while paused) the user can drag the preview actor around to simulate the interaction from different start positions
		if (AActor* SelectedActor = GetSelectedActor())
		{
			SelectedActor->SetActorLocationAndRotation(SelectedActor->GetActorLocation() + InDrag, SelectedActor->GetActorRotation() + InRot);
			return true;
		}
	}
	else if (IsSimulateModeInactive())
	{
		if (UContextualAnimSelectionCriterion_TriggerArea* Spatial = Cast<UContextualAnimSelectionCriterion_TriggerArea>(GetSelectedSelectionCriterion()))
		{
			FMatrix WidgetCoordSystem = FMatrix::Identity;
			GetCustomDrawingCoordinateSystem(WidgetCoordSystem, nullptr);

			InDrag = WidgetCoordSystem.InverseTransformVector(InDrag);

			FVector& Point = Spatial->PolygonPoints[SelectionInfo.Criterion.Value >= 4 ? SelectionInfo.Criterion.Value - 4 : SelectionInfo.Criterion.Value];
			Point.X += InDrag.X;
			Point.Y += InDrag.Y;

			if (InDrag.Z != 0.f)
			{
				if (SelectionInfo.Criterion.Value < 4)
				{
					for (int32 Idx = 0; Idx < Spatial->PolygonPoints.Num(); Idx++)
					{
						Spatial->PolygonPoints[Idx].Z += InDrag.Z;
					}

					Spatial->Height = FMath::Max(Spatial->Height - InDrag.Z, 0.f);
				}
				else
				{
					Spatial->Height = FMath::Max(Spatial->Height + InDrag.Z, 0.f);
				}
			}

			return true;
		}
		else if (ModifyingActorTransformInSceneState != EModifyActorTransformInSceneState::Inactive)
		{
			if (WantsToModifyMeshToSceneForSelectedActor())
			{
				AActor* SelectedActor = GetSelectedActor();
				FContextualAnimTrack* SelectedTrack = GetSelectedAnimTrack();
				if (SelectedActor && SelectedTrack)
				{
					SelectedTrack->MeshToScene.SetLocation(SelectedTrack->MeshToScene.GetLocation() + InDrag);
					SelectedTrack->MeshToScene.SetRotation(FQuat(InRot) * SelectedTrack->MeshToScene.GetRotation());

					SelectedActor->SetActorLocationAndRotation(SelectedActor->GetActorLocation() + InDrag, SelectedActor->GetActorRotation() + InRot);				
				}
			}

			return true;
		}
	}

	return false;
}

bool FContextualAnimViewModel::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (SelectionInfo.Criterion.Value != INDEX_NONE)
	{
		FTransform PrimaryActorTransform = FTransform::Identity;
		if (const FContextualAnimSceneBinding* Binding = SceneBindings.FindBindingByRole(GetSceneAsset()->GetPrimaryRole()))
		{
			PrimaryActorTransform = Binding->GetTransform();
			InMatrix = PrimaryActorTransform.ToMatrixNoScale().RemoveTranslation();
			return true;
		}
	}
	else if (AActor* SelectedActor = GetSelectedActor())
	{
		InMatrix = SelectedActor->GetActorTransform().ToMatrixNoScale().RemoveTranslation();
		return true;
	}

	return false;
}

bool FContextualAnimViewModel::WantsToModifyMeshToSceneForSelectedActor() const
{
	//@TODO: Communicate this to the user somehow
	return FSlateApplication::Get().GetModifierKeys().IsShiftDown();
}

bool FContextualAnimViewModel::ShouldPreviewSceneDrawWidget() const
{
	// When Simulate Mode is inactive we show the widget if an editable selection criterion is selected (only Trigger Area for now) 
	// or if an actor is selected and user wants to modify the MeshToScene transform
	if (IsSimulateModeInactive())
	{
		const UContextualAnimSelectionCriterion* SelectionCriterion = GetSelectedSelectionCriterion();
		return ((SelectionCriterion && SelectionCriterion->GetClass()->IsChildOf<UContextualAnimSelectionCriterion_TriggerArea>()) ||
			(WantsToModifyMeshToSceneForSelectedActor() && GetSelectedActor()));
	}
	// When Simulate Mode is Paused we show the widget if an actor is selected, so the user can modify the position of any of the actor before triggering the interaction
	else if (IsSimulateModePaused())
	{
		return GetSelectedActor() != nullptr;
	}

	return false;
}

FVector FContextualAnimViewModel::GetWidgetLocationFromSelection() const
{
	if (SelectionInfo.Criterion.Value != INDEX_NONE)
	{
		if (const UContextualAnimSelectionCriterion_TriggerArea* Spatial = Cast<UContextualAnimSelectionCriterion_TriggerArea>(GetSelectedSelectionCriterion()))
		{
			FVector Location = FVector::ZeroVector;
			if (SelectionInfo.Criterion.Value < 4)
			{
				Location = Spatial->PolygonPoints[SelectionInfo.Criterion.Value];
			}
			else
			{
				Location = Spatial->PolygonPoints[SelectionInfo.Criterion.Value - 4] + FVector::UpVector * Spatial->Height;
			}

			FTransform PrimaryActorTransform = FTransform::Identity;
			if (const FContextualAnimSceneBinding* Binding = SceneBindings.FindBindingByRole(SceneAsset->GetPrimaryRole()))
			{
				PrimaryActorTransform = Binding->GetTransform();
			}

			return PrimaryActorTransform.TransformPositionNoScale(Location);
		}
	}
	else if (AActor* SelectedActor = GetSelectedActor())
	{
		return SelectedActor->GetActorLocation();
	}

	return FVector::ZeroVector;
}

bool FContextualAnimViewModel::StartTracking()
{
	if (IsSimulateModeInactive() && WantsToModifyMeshToSceneForSelectedActor())
	{
		if (const FContextualAnimSceneBinding* Binding = GetSelectedBinding())
		{
			if (ModifyingActorTransformInSceneState == EModifyActorTransformInSceneState::Inactive)
			{
				ModifyingActorTransformInSceneState = EModifyActorTransformInSceneState::Modifying;
				ModifyingTransformInSceneCachedActor = Binding->GetActor();

				GEditor->BeginTransaction(LOCTEXT("ModifyMeshToSceneTransform", "Modify MeshToScene Transform"));
				SceneAsset->Modify();
			}

			return true;
		}
	}

	return false;
}

bool FContextualAnimViewModel::EndTracking()
{
	if (ModifyingActorTransformInSceneState == EModifyActorTransformInSceneState::Modifying)
	{
		GEditor->EndTransaction();
		
		ModifyingActorTransformInSceneState = EModifyActorTransformInSceneState::Inactive;
		ModifyingTransformInSceneCachedActor.Reset();

		return true;		
	}

	return false;
}

void FContextualAnimViewModel::UpdateRoles()
{
	FScopedTransaction Transaction(LOCTEXT("CAS_UpdateRoles", "UpdateRoles"));
	SceneAsset->Modify();

	TArray<FName> ValidRoles;
	if (SceneAsset)
	{
		ValidRoles = SceneAsset->GetRoles();
	}

	for(FContextualAnimSceneSection& Section : SceneAsset->Sections)
	{
		for (FContextualAnimSet& AnimSet : Section.AnimSets)
		{
			AnimSet.Tracks.RemoveAll([&ValidRoles](const FContextualAnimTrack& Track)->bool
				{
					return ValidRoles.Contains(Track.Role) == false;
				});
		}
	}

	SetDefaultMode();
}

void FContextualAnimViewModel::CacheWarpPoints()
{
	for(FContextualAnimSceneSection& Section : SceneAsset->Sections)
	{
		for (FContextualAnimSet& AnimSet : Section.AnimSets)
		{
			AnimSet.WarpPoints.Reset();

			for (const FContextualAnimWarpPointDefinition& WarpPointDef : Section.WarpPointDefinitions)
			{
				if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::PrimaryActor)
				{
					const FName PrimaryRole = SceneAsset->GetPrimaryRole();
					const FContextualAnimTrack* AnimTrack = AnimSet.Tracks.FindByPredicate([PrimaryRole](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role == PrimaryRole; });
					if (AnimTrack)
					{
						const FTransform RootTransform = AnimTrack->Animation ? UContextualAnimUtilities::ExtractRootTransformFromAnimation(AnimTrack->Animation, 0.f) : FTransform::Identity;
						const FTransform WarpPointTransform = (FTransform(SceneAsset->GetMeshToComponentForRole(AnimTrack->Role).GetRotation()).Inverse() * (RootTransform * AnimTrack->MeshToScene));
						AnimSet.WarpPoints.Add(WarpPointDef.WarpTargetName, WarpPointTransform);
					}
				}
				else if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::Socket)
				{
					if (const FContextualAnimSceneBinding* Primary = SceneBindings.GetPrimaryBinding())
					{
						if (UMeshComponent* MeshComp = UContextualAnimUtilities::TryGetMeshComponentWithSocket(Primary->GetActor(), WarpPointDef.SocketName))
						{
							const FTransform WarpPointTransform = MeshComp->GetSocketTransform(WarpPointDef.SocketName, ERelativeTransformSpace::RTS_Actor);
							AnimSet.WarpPoints.Add(WarpPointDef.WarpTargetName, WarpPointTransform);
						}
					}
				}
				else if (WarpPointDef.Mode == EContextualAnimWarpPointDefinitionMode::Custom)
				{
					const FContextualAnimWarpPointCustomParams& Params = WarpPointDef.Params;

					FTransform WarpPointTransform = FTransform::Identity;
					FContextualAnimTrack* AnimTrack = AnimSet.Tracks.FindByPredicate([&Params](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role == Params.Origin; });
					if (AnimTrack)
					{
						if (Params.bAlongClosestDistance)
						{
							FContextualAnimTrack* OtherAnimTrack = AnimSet.Tracks.FindByPredicate([&Params](const FContextualAnimTrack& AnimTrack) { return AnimTrack.Role == Params.OtherRole; });
							if (OtherAnimTrack)
							{
								FTransform T1 = AnimTrack->Animation ? UContextualAnimUtilities::ExtractRootTransformFromAnimation(AnimTrack->Animation, 0.f) : FTransform::Identity;
								T1 = (FTransform(SceneAsset->GetMeshToComponentForRole(AnimTrack->Role).GetRotation()).Inverse() * (T1 * AnimTrack->MeshToScene));

								FTransform T2 = OtherAnimTrack->Animation ? UContextualAnimUtilities::ExtractRootTransformFromAnimation(OtherAnimTrack->Animation, 0.f) : FTransform::Identity;
								T2 = (FTransform(SceneAsset->GetMeshToComponentForRole(OtherAnimTrack->Role).GetRotation()).Inverse() * (T2 * OtherAnimTrack->MeshToScene));

								WarpPointTransform.SetLocation(FMath::Lerp<FVector>(T1.GetLocation(), T2.GetLocation(), Params.Weight));
								WarpPointTransform.SetRotation((T2.GetLocation() - T1.GetLocation()).GetSafeNormal2D().ToOrientationQuat());
							}
						}
						else
						{
							const FTransform RootTransform = AnimTrack->Animation ? UContextualAnimUtilities::ExtractRootTransformFromAnimation(AnimTrack->Animation, 0.f) : FTransform::Identity;
							WarpPointTransform = (FTransform(SceneAsset->GetMeshToComponentForRole(AnimTrack->Role).GetRotation()).Inverse() * (RootTransform * AnimTrack->MeshToScene));
						}
					}

					AnimSet.WarpPoints.Add(WarpPointDef.WarpTargetName, WarpPointTransform);
				}
			}
		}
	}
}

void FContextualAnimViewModel::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = (PropertyChangedEvent.Property != nullptr) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != nullptr) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	UE_LOGF(LogContextualAnim, Verbose, "FContextualAnimViewModel::OnFinishedChangingProperties MemberPropertyName: %ls PropertyName: %ls", *MemberPropertyName.ToString(), *PropertyName.ToString());

	// Validate PreviewActorClass has the required UContextualAnimSceneActorComponent
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UContextualAnimSceneAsset, OverridePreviewData) &&
		PropertyName == GET_MEMBER_NAME_CHECKED(FContextualAnimActorPreviewData, PreviewActorClass))
	{
		bool bReverted = false;
		for (FContextualAnimActorPreviewData& PreviewData : SceneAsset->OverridePreviewData)
		{
			if (PreviewData.Type == EContextualAnimActorPreviewType::Actor)
			{
				if (UClass* ActorClass = PreviewData.PreviewActorClass.LoadSynchronous())
				{
					// Check native components on the CDO
					bool bHasSceneActorComponent = false;
					if (AActor* CDO = ActorClass->GetDefaultObject<AActor>())
					{
						bHasSceneActorComponent = CDO->FindComponentByClass<UContextualAnimSceneActorComponent>() != nullptr;
					}

					// Check Blueprint SCS for components added in the Blueprint editor
					if (!bHasSceneActorComponent)
					{
						for (UClass* CurrentClass = ActorClass; CurrentClass && !bHasSceneActorComponent; CurrentClass = CurrentClass->GetSuperClass())
						{
							if (UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(CurrentClass))
							{
								if (USimpleConstructionScript* SCS = BPGC->SimpleConstructionScript)
								{
									for (USCS_Node* Node : SCS->GetAllNodes())
									{
										if (Node && Node->ComponentTemplate && Node->ComponentTemplate->IsA<UContextualAnimSceneActorComponent>())
										{
											bHasSceneActorComponent = true;
											break;
										}
									}
								}
							}
						}
					}

					if (!bHasSceneActorComponent)
					{
						ShowMissingSceneActorComponentWarning(ActorClass->GetName());

						PreviewData.PreviewActorClass.Reset();
						bReverted = true;
					}
				}
			}
		}

		if (bReverted)
		{
			SceneAsset->MarkPackageDirty();
			RefreshDetailsView();
		}
		else
		{
			SetDefaultMode();
		}
		return;
	}

	// Refresh preview scene when Preview data changes
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UContextualAnimSceneAsset, OverridePreviewData) || 
		MemberPropertyName == GET_MEMBER_NAME_CHECKED(UContextualAnimSceneAsset, MultiSectionPreviewSequence))
	{
		SetDefaultMode();
		return;
	}

	// Refresh preview scene if necessary
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UContextualAnimSceneAsset, Sections) &&
		(PropertyName == GET_MEMBER_NAME_CHECKED(UContextualAnimSceneAsset, Sections) ||
		 PropertyName == GET_MEMBER_NAME_CHECKED(FContextualAnimSceneSection, AnimSets) ||
		 PropertyName == GET_MEMBER_NAME_CHECKED(FContextualAnimTrack, Animation) ||
		 PropertyName == GET_MEMBER_NAME_CHECKED(FContextualAnimTrack, MeshToScene)))
	{
		SetDefaultMode();
	}
}

bool FContextualAnimViewModel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	// Ensure that we only react to modifications to the SceneAsset
	if (SceneAsset)
	{
		for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjectContexts)
		{
			UObject* Object = TransactionObjectPair.Key;
			while (Object != nullptr)
			{
				if (Object == SceneAsset)
				{
					return true;
				}

				Object = Object->GetOuter();
			}
		}
	}

	return false;
}

void FContextualAnimViewModel::PostUndo(bool bSuccess)
{
	// Refresh everything after a Undo operation
	SetDefaultMode();
}

void FContextualAnimViewModel::PostRedo(bool bSuccess)
{
	// Refresh everything after a Redo operation
	SetDefaultMode();
}

#undef LOCTEXT_NAMESPACE
