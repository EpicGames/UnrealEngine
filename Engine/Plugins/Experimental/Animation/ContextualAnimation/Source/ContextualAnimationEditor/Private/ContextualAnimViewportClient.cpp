// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimViewportClient.h"
#include "ContextualAnimAssetEditorToolkit.h"
#include "ContextualAnimPreviewScene.h"
#include "SContextualAnimViewport.h"
#include "AssetEditorModeManager.h"
#include "UnrealWidget.h"
#include "ContextualAnimEdMode.h"

FContextualAnimViewportClient::FContextualAnimViewportClient(const TSharedRef<FContextualAnimPreviewScene>& InPreviewScene, const TSharedRef<SContextualAnimViewport>& InViewport, const TSharedRef<FContextualAnimAssetEditorToolkit>& InAssetEditorToolkit)
	: FEditorViewportClient(nullptr, &InPreviewScene.Get(), StaticCastSharedRef<SEditorViewport>(InViewport))
	, PreviewScenePtr(InPreviewScene)
	, AssetEditorToolkitPtr(InAssetEditorToolkit)
{
	Widget->SetUsesEditorModeTools(ModeTools.Get());
	StaticCastSharedPtr<FAssetEditorModeManager>(ModeTools)->SetPreviewScene(&InPreviewScene.Get());
	ModeTools->SetDefaultMode(FContextualAnimEdMode::EdModeId);

	SetRealtime(true);

	SetWidgetCoordSystemSpace(COORD_Local);
	ModeTools->SetWidgetMode(UE::Widget::WM_Translate);
}

void FContextualAnimViewportClient::TrackingStarted(const struct FInputEventState& InInputState, bool bIsDraggingWidget, bool bNudge)
{
	ModeTools->StartTracking(this, Viewport);
}

void FContextualAnimViewportClient::TrackingStopped()
{
	ModeTools->EndTracking(this, Viewport);
	Invalidate();
}

void FContextualAnimViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FEditorViewportClient::Draw(View, PDI);
}