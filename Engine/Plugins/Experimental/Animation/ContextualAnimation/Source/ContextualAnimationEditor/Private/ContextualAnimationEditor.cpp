// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContextualAnimationEditor.h"
#include "ContextualAnimEdMode.h"
#include "ContextualAnimEditorStyle.h"
#include "ContextualAnimTypeActions.h"
#include "ContextualAnimMovieSceneTrackEditor.h"
#include "ContextualAnimMovieSceneNotifyTrackEditor.h"
#include "ContextualAnimAssetEditorCommands.h"
#include "DetailCustomizations/ContextualAnimNotifySectionDetailCustom.h"
#include "EditorModeRegistry.h"
#include "ISequencerModule.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FContextualAnimationEditorModule"

void FContextualAnimationEditorModule::StartupModule()
{
	// Register Asset Editor Commands
	FContextualAnimAssetEditorCommands::Register();

	FContextualAnimEditorStyle::Initialize();

	// Register Ed Mode used to interact with the preview scene
	FEditorModeRegistry::Get().RegisterMode<FContextualAnimEdMode>(FContextualAnimEdMode::EdModeId, 
		LOCTEXT("ContextualAnimEdModeEdModeName", "ContextualAnim"), 
		FSlateIcon(FContextualAnimEditorStyle::GetStyleSetName(), "ContextualAnimEditor.Icon", "ContextualAnimEditor.Icon"),
		false,
		9000);

	// Register Contextual Anim Scene Asset Type Actions 
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	ContextualAnimAssetActions = MakeShared<FContextualAnimTypeActions>();
	AssetTools.RegisterAssetTypeActions(ContextualAnimAssetActions.ToSharedRef());

	// Register MovieSceneAnimNotifyTrackEditor
	ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked< ISequencerModule >("Sequencer");
	MovieSceneAnimNotifyTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FContextualAnimMovieSceneNotifyTrackEditor::CreateTrackEditor));
	MovieSceneAnimNotifyTrackEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FContextualAnimMovieSceneTrackEditor::CreateTrackEditor));

	// Register Detail Customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");;
	PropertyModule.RegisterCustomClassLayout("ContextualAnimMovieSceneNotifySection", FOnGetDetailCustomizationInstance::CreateStatic(&FContextualAnimNotifySectionDetailCustom::MakeInstance));
}

void FContextualAnimationEditorModule::ShutdownModule()
{
	// Unregister Asset Editor Commands
	FContextualAnimAssetEditorCommands::Unregister();

	// Unregister MovieSceneAnimNotifyTrackEditor
	if (FModuleManager::Get().IsModuleLoaded("Sequencer"))
	{
		ISequencerModule& SequencerModule = FModuleManager::Get().GetModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.UnRegisterTrackEditor(MovieSceneAnimNotifyTrackEditorHandle);
	}

	// Unregister Contextual Anim Scene Asset Type Actions 
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.UnregisterAssetTypeActions(ContextualAnimAssetActions.ToSharedRef());
	}

	// Unregister Detail Customizations
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("ContextualAnimMovieSceneNotifySection");
	}

	// Unregister Ed Mode
	FEditorModeRegistry::Get().UnregisterMode(FContextualAnimEdMode::EdModeId);

	FContextualAnimEditorStyle::Shutdown();
}

FContextualAnimationEditorModule& FContextualAnimationEditorModule::Get()
{
	return FModuleManager::Get().GetModuleChecked<FContextualAnimationEditorModule>("ContextualAnimationEditor");
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FContextualAnimationEditorModule, ContextualAnimationEditor)
