// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/ContextualAnimNotifySectionDetailCustom.h"
#include "Animation/AnimTypes.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Animation/AnimNotifies/AnimNotify.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "ContextualAnimMovieSceneNotifySection.h"
#include "IDetailPropertyRow.h"

TSharedRef<IDetailCustomization> FContextualAnimNotifySectionDetailCustom::MakeInstance()
{
	return MakeShared<FContextualAnimNotifySectionDetailCustom>();
}

void FContextualAnimNotifySectionDetailCustom::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectList;
	DetailBuilder.GetObjectsBeingCustomized(ObjectList);
	check(ObjectList.Num() > 0);

	Section = Cast<UContextualAnimMovieSceneNotifySection>(ObjectList[0].Get());
	check(Section.IsValid());

	// Add new category to show the properties from the actual AnimNotify object
	if (const FAnimNotifyEvent* NotifyEvent = Section->GetAnimNotifyEvent())
	{
		UObject* NotifyObject = nullptr;
		if (NotifyEvent->Notify)
		{
			NotifyObject = NotifyEvent->Notify;
		}
		else
		{
			NotifyObject = NotifyEvent->NotifyStateClass;
		}

		if (NotifyObject)
		{
			IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("Notify"), FText::GetEmpty(), ECategoryPriority::Variable);

			TArray<UObject*> ExternalObjects = { NotifyObject };
			IDetailPropertyRow* PropertyRow = Category.AddExternalObjects(ExternalObjects);
			PropertyRow->ShouldAutoExpand(true);
		}
	}
}
