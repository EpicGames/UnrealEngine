// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailCustomizations/ContextualAnimSceneAssetDetailCustom.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SButton.h"
#include "Misc/MessageDialog.h"
#include "ContextualAnimSceneAsset.h"
#include "ContextualAnimViewModel.h"

#define LOCTEXT_NAMESPACE "ContextualAnimSceneAssetDetailCustom"

TSharedRef<IDetailCustomization> FContextualAnimSceneAssetDetailCustom::MakeInstance(TSharedRef<FContextualAnimViewModel> ViewModelRef)
{
	return MakeShared<FContextualAnimSceneAssetDetailCustom>(ViewModelRef);
}

FContextualAnimSceneAssetDetailCustom::FContextualAnimSceneAssetDetailCustom(TSharedRef<FContextualAnimViewModel> ViewModelRef)
	: ViewModelPtr(ViewModelRef)
{
	
}

void FContextualAnimSceneAssetDetailCustom::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	check(ViewModelPtr.IsValid());
	TSharedPtr<FContextualAnimViewModel> ViewModel = ViewModelPtr.Pin();

	TArray<TWeakObjectPtr<UObject>> ObjectList;
	DetailBuilder.GetObjectsBeingCustomized(ObjectList);
	check(ObjectList.Num() > 0);

	const UContextualAnimSceneAsset* SceneAsset = Cast<const UContextualAnimSceneAsset>(ObjectList[0].Get());
	check(SceneAsset);

	IDetailCategoryBuilder& SettingsCategory = DetailBuilder.EditCategory(TEXT("Settings"));
	SettingsCategory.SetSortOrder(0);

	// Add button to update role tracks for the scene
	SettingsCategory.AddCustomRow(FText::GetEmpty())
		.ValueContent()
		.VAlign(VAlign_Center)
		.MaxDesiredWidth(250)
		[
			SNew(SButton).Text(LOCTEXT("UpdateRolesPointsLabel", "Update Roles"))
			.OnClicked_Lambda([ViewModel]()
				{
					const FText DialogMsg = LOCTEXT("UpdateRolesPointsDialog", "Updating the roles will remove any tracks for roles that don't exist in your Roles Asset. Are you sure you want to continue?");
					if (FMessageDialog::Open(EAppMsgType::YesNo, DialogMsg) == EAppReturnType::Yes)
					{
						ViewModel->UpdateRoles();
					}

					return FReply::Handled();
				})
		];

	IDetailCategoryBuilder& PreviewCategory = DetailBuilder.EditCategory(TEXT("Preview"));
	PreviewCategory.SetSortOrder(1);

	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory(TEXT("Defaults"));
	Category.SetCategoryVisibility(false);

	const int32 SectionIdx = ViewModel->GetDetailsSectionIdx();
	if (const FContextualAnimSceneSection* Section = SceneAsset->GetSection(SectionIdx))
	{
		// Create new category to show editable data only for the currently active section in the editor
		IDetailCategoryBuilder& CurrentSectionCategory = DetailBuilder.EditCategory(TEXT("Active Section"));

		// Get IPropertyHandle for the array of Sections
		TSharedPtr<IPropertyHandle> SectionsPropertyHandle = DetailBuilder.GetProperty(FName(TEXT("Sections")));
		SectionsPropertyHandle->MarkHiddenByCustomization();

		// Get IPropertyHandle for the current Section
		TSharedRef<IPropertyHandle> SectionPropertyHandle = SectionsPropertyHandle->GetChildHandle(SectionIdx).ToSharedRef();

		// Show editable properties in the custom category
		CurrentSectionCategory.AddProperty(SectionPropertyHandle->GetChildHandle(FName(TEXT("Name"))).ToSharedRef());
		CurrentSectionCategory.AddProperty(SectionPropertyHandle->GetChildHandle(FName(TEXT("WarpPointDefinitions"))).ToSharedRef());
		CurrentSectionCategory.AddProperty(SectionPropertyHandle->GetChildHandle(FName(TEXT("bSyncAnimations"))).ToSharedRef());

		// Add button to generate warp points for the scene
		CurrentSectionCategory.AddCustomRow(FText::GetEmpty())
			.ValueContent()
			.VAlign(VAlign_Center)
			.MaxDesiredWidth(250)
			[
				SNew(SButton).Text(LOCTEXT("UpdateWarpPointsLabel", "Update Warp Points"))
				.OnClicked_Lambda([ViewModel]()
					{
						const FText DialogMsg = LOCTEXT("RefreshWarpPointsDialog", "Warp Points should be updated while previewing the scene with the meshes the animations where originally authored for. Are you sure you want to continue?");
						if (FMessageDialog::Open(EAppMsgType::YesNo, DialogMsg) == EAppReturnType::Yes)
						{
							ViewModel->CacheWarpPoints();
						}

						return FReply::Handled();
					})
			];

		// Add button to remove the entire section
		CurrentSectionCategory.AddCustomRow(FText::GetEmpty())
			.ValueContent()
			.VAlign(VAlign_Center)
			.MaxDesiredWidth(250)
			[
				SNew(SButton).Text(LOCTEXT("RemoveSectionButtonLabel", "Remove Section"))
				.OnClicked_Lambda([ViewModel, SectionIdx]()
				{
					if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveSectionDialog", "Are you sure you want to remove this Section?")) == EAppReturnType::Yes)
					{
						ViewModel->RemoveSection(SectionIdx);
					}

					return FReply::Handled();
				})
			];

		// Same as above but for the currently active AnimSet
		const int32 AnimSetIdx = ViewModel->GetDetailsAnimSetIdx();
		if (const FContextualAnimSet* AnimSet = Section->GetAnimSet(AnimSetIdx))
		{
			// Create new category to show editable data only for the currently active AnimSet in the editor
			IDetailCategoryBuilder& CurrentAnimSetCategory = DetailBuilder.EditCategory(TEXT("Active AnimSet"));

			// Get IPropertyHandle for the array of AnimSet
			TSharedPtr<IPropertyHandle> AnimSetArrayPropertyHandle = SectionPropertyHandle->GetChildHandle(FName(TEXT("AnimSets")));

			// Get IPropertyHandle for the current AnimSet
			TSharedRef<IPropertyHandle> AnimSetPropertyHandle = AnimSetArrayPropertyHandle->GetChildHandle(AnimSetIdx).ToSharedRef();

			// Show general AnimSet properties in a custom category
			CurrentAnimSetCategory.AddProperty(AnimSetPropertyHandle->GetChildHandle(FName(TEXT("Name"))).ToSharedRef());
			CurrentAnimSetCategory.AddProperty(AnimSetPropertyHandle->GetChildHandle(FName(TEXT("RandomWeight"))).ToSharedRef());

			// Add button to remove the entire set
			CurrentAnimSetCategory.AddCustomRow(FText::GetEmpty())
				.ValueContent()
				.VAlign(VAlign_Center)
				.MaxDesiredWidth(250)
				[
					SNew(SButton).Text(LOCTEXT("RemoveAnimSetButtonLabel", "Remove AnimSet"))
					.OnClicked_Lambda([ViewModel, SectionIdx, AnimSetIdx]()
					{
						if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveAnimSetDialog", "Are you sure you want to remove this AnimSet?")) == EAppReturnType::Yes)
						{
							ViewModel->RemoveAnimSet(SectionIdx, AnimSetIdx);
						}
						
						return FReply::Handled();
					})
				];

			// Same as above but for the selected AnimTrack
			if (const FContextualAnimTrack* SelectedAnimTrack = ViewModel->GetSelectedAnimTrack())
			{
				// Create new category to show editable data only for the currently selected track
				IDetailCategoryBuilder& AnimTrackCategory = DetailBuilder.EditCategory(TEXT("Selected AnimTrack"));

				// Get IPropertyHandle for the array of AnimTracks
				TSharedPtr<IPropertyHandle> TracksPropertyHandle = AnimSetPropertyHandle->GetChildHandle(FName(TEXT("Tracks")));

				// Get IPropertyHandle for the selected AnimTrack
				TSharedRef<IPropertyHandle> AnimTrackPropertyHandle = TracksPropertyHandle->GetChildHandle(SelectedAnimTrack->AnimTrackIdx).ToSharedRef();
				AnimTrackPropertyHandle->MarkHiddenByCustomization();

				// Show all the editable properties of the AnimTrack in a custom category
				uint32 AnimTrackChildren = 0;
				AnimTrackPropertyHandle->GetNumChildren(AnimTrackChildren);
				for (uint32 Index = 0; Index < AnimTrackChildren; ++Index)
				{
					AnimTrackCategory.AddProperty(AnimTrackPropertyHandle->GetChildHandle(Index).ToSharedRef());
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
