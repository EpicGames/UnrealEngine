// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContextualAnimAssetBrowser.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "Widgets/SBoxPanel.h"
#include "UObject/AssetRegistryTagsContext.h"

void SContextualAnimAssetBrowser::Construct(const FArguments& InArgs)
{
	ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(AssetBrowserBox, SBox)
			]
		];

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimMontage::StaticClass()->GetClassPathName());
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SContextualAnimAssetBrowser::OnAssetDoubleClicked);
	//AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SContextualAnimAssetBrowser::OnShouldFilterAsset);
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = false;

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
}

void SContextualAnimAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (!AssetData.GetAsset())
	{
		return;
	}

	// @TODO: Show pop up to add new role to the interaction using this animation
	//UAnimMontage* Animation = Cast<UAnimMontage>(AssetData.GetAsset());
}

bool SContextualAnimAssetBrowser::OnShouldFilterAsset(const FAssetData& AssetData)
{
	if (!AssetData.GetClass()->IsChildOf(UAnimMontage::StaticClass()))
	{
		return true;
	}

	return false;
}
