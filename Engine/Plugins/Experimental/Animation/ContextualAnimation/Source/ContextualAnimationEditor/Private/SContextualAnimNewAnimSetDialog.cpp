// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContextualAnimNewAnimSetDialog.h"
#include "DetailsViewArgs.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "IStructureDetailsView.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "ContextualAnimEditorTypes.h"
#include "ContextualAnimViewModel.h"
#include "ContextualAnimSceneAsset.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ContextualAnimNewAnimSetDialog"

void SContextualAnimNewAnimSetDialog::Construct(const FArguments& InArgs, const TSharedRef<FContextualAnimViewModel>& InViewModel)
{
	WeakViewModel = InViewModel;

	WidgetStruct = MakeShared<FStructOnScope>(FContextualAnimNewAnimSetParams::StaticStruct());
	FContextualAnimNewAnimSetParams* Params = (FContextualAnimNewAnimSetParams*)WidgetStruct->GetStructMemory();

	const TArray<FName> Roles = InViewModel->GetSceneAsset()->GetRoles();
	for (FName Role : Roles)
	{
		FContextualAnimNewAnimSetData Entry;
		Entry.RoleName = Role;
		Params->Data.Add(Entry);
	}

	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.bAllowSearch = false;
	Args.bAllowFavoriteSystem = false;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyModule.CreateStructureDetailView(Args, FStructureDetailsViewArgs(), WidgetStruct);

	SWindow::Construct(SWindow::FArguments()
		.Title(LOCTEXT("NewAnimSetTitle", "Add New AnimSet"))
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.ClientSize(FVector2D(500, 400))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(3)
			[
				StructureDetailsView->GetWidget().ToSharedRef()
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked(this, &SContextualAnimNewAnimSetDialog::OnButtonClick, EAppReturnType::Ok)
				]
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked(this, &SContextualAnimNewAnimSetDialog::OnButtonClick, EAppReturnType::Cancel)
				]
			]
		]);
}


FReply SContextualAnimNewAnimSetDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;

	if(ButtonID == EAppReturnType::Ok)
	{
		FContextualAnimNewAnimSetParams* Params = (FContextualAnimNewAnimSetParams*) WidgetStruct->GetStructMemory();
		check(Params);

		WeakViewModel.Pin()->AddNewAnimSet(*Params);
	}

	RequestDestroyWindow();

	return FReply::Handled();
}

EAppReturnType::Type SContextualAnimNewAnimSetDialog::Show()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

#undef LOCTEXT_NAMESPACE
