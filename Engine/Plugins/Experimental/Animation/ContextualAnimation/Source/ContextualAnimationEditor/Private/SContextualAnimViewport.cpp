// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContextualAnimViewport.h"
#include "ContextualAnimViewportClient.h"
#include "Editor/UnrealEdTypes.h"
#include "SContextualAnimViewportToolbar.h"
#include "ContextualAnimAssetEditorCommands.h"
#include "ContextualAnimEditorStyle.h"
#include "ContextualAnimAssetEditorToolkit.h"
#include "ContextualAnimViewModel.h"
#include "EditorFontGlyphs.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UICommandList.h"
#include "Viewports.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ContextualAnimViewport"

void SContextualAnimViewport::Construct(const FArguments& InArgs, const FContextualAnimViewportRequiredArgs& InRequiredArgs)
{
	PreviewScenePtr = InRequiredArgs.PreviewScene;
	AssetEditorToolkitPtr = InRequiredArgs.AssetEditorToolkit;

	SEditorViewport::Construct(
		SEditorViewport::FArguments()
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute())
		.AddMetaData<FTagMetaData>(TEXT("AnimationTools.Viewport"))
	);

	auto GetNotificationText = [this]()
	{
		return LOCTEXT("MeshToSceneChangeConfirmText", "Apply changes to the transform of the actor in the scene?");
	};

	auto GetNotificationVisibility = [this]()
	{
		return EVisibility::Hidden;
	};

	auto ApplyChanges = [this]()
	{
		return FReply::Handled();
	};

	auto CancelChanges = [this]()
 	{
		return FReply::Handled();
	};

	ViewportOverlay->AddSlot()
	[
		SNew(SVerticalBox)
		.Visibility_Lambda(GetNotificationVisibility)
		+ SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SOverlay)
			+ SOverlay::Slot()
			.Padding(8)
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Right)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("AnimViewport.Notification.Warning"))
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(4.0f, 4.0f)
					[
						SNew(SHorizontalBox)
						.ToolTipText_Lambda(GetNotificationText)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
							.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
							.Text(FEditorFontGlyphs::Exclamation_Triangle)
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.0f)
						[
							SNew(STextBlock)
							.Text_Lambda(GetNotificationText)
							.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SButton)
						.ForegroundColor(FSlateColor::UseForeground())
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
						.ToolTipText_Lambda(GetNotificationText)
						.OnClicked_Lambda(ApplyChanges)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 4.0f, 0.0f)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
								.Text(FEditorFontGlyphs::Pencil)
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Text(LOCTEXT("ApplyButtonText", "Apply"))
							]
						]
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.0f, 0.0f)
					[
						SNew(SButton)
						.ForegroundColor(FSlateColor::UseForeground())
						.ButtonStyle(FAppStyle::Get(), "FlatButton.Danger")
						.ToolTipText(LOCTEXT("DiscardMeshToSceneChangeToolTip", "Discard changes to the transform of the actor in the scene."))
						.OnClicked_Lambda(CancelChanges)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 4.0f, 0.0f)
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
								.Text(FEditorFontGlyphs::Times)
							]
							+ SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
								.Text(LOCTEXT("CancelButtonText", "Cancel"))
							]
						]
					]
				]
			]
		]
	];
}

const FSlateBrush* SContextualAnimViewport::OnGetViewportBorderBrush() const
{
	// Highlight the border of the viewport when Simulate Mode is active
	if (AssetEditorToolkitPtr.Pin()->IsSimulateModeActive())
	{
		return FContextualAnimEditorStyle::Get().GetBrush("ContextualAnimEditor.Viewport.Border");
	}

	return nullptr;
}

FSlateColor SContextualAnimViewport::OnGetViewportBorderColorAndOpacity() const
{
	return FLinearColor::Yellow;
}

void SContextualAnimViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	const FContextualAnimAssetEditorCommands& Commands = FContextualAnimAssetEditorCommands::Get();

	TSharedRef<FContextualAnimViewportClient> ViewportClientRef = ViewportClient.ToSharedRef();

	// Show IK Targets Options
	{
		CommandList->MapAction(
			Commands.ShowIKTargetsDrawAll,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetIKTargetsDrawMode, EMultiOptionDrawMode::All),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsIKTargetsDrawModeSet, EMultiOptionDrawMode::All));

		CommandList->MapAction(
			Commands.ShowIKTargetsDrawSelected,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetIKTargetsDrawMode, EMultiOptionDrawMode::Single),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsIKTargetsDrawModeSet, EMultiOptionDrawMode::Single));

		CommandList->MapAction(
			Commands.ShowIKTargetsDrawNone,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetIKTargetsDrawMode, EMultiOptionDrawMode::None),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsIKTargetsDrawModeSet, EMultiOptionDrawMode::None));
	}

	// Show Selection Criteria Options
	{
		CommandList->MapAction(
			Commands.ShowSelectionCriteriaAllSets,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetSelectionCriteriaDrawMode, EMultiOptionDrawMode::All),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsSelectionCriteriaDrawModeSet, EMultiOptionDrawMode::All));

		CommandList->MapAction(
			Commands.ShowSelectionCriteriaActiveSet,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetSelectionCriteriaDrawMode, EMultiOptionDrawMode::Single),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsSelectionCriteriaDrawModeSet, EMultiOptionDrawMode::Single));

		CommandList->MapAction(
			Commands.ShowSelectionCriteriaNone,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetSelectionCriteriaDrawMode, EMultiOptionDrawMode::None),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsSelectionCriteriaDrawModeSet, EMultiOptionDrawMode::None));
	}

	// Show Entry Poses Options
	{
		CommandList->MapAction(
			Commands.ShowEntryPosesAllSets,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetEntryPosesDrawMode, EMultiOptionDrawMode::All),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsEntryPosesDrawModeSet, EMultiOptionDrawMode::All));

		CommandList->MapAction(
			Commands.ShowEntryPosesActiveSet,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetEntryPosesDrawMode, EMultiOptionDrawMode::Single),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsEntryPosesDrawModeSet, EMultiOptionDrawMode::Single));

		CommandList->MapAction(
			Commands.ShowEntryPosesNone,
			FExecuteAction::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::SetEntryPosesDrawMode, EMultiOptionDrawMode::None),
			FCanExecuteAction(),
			FIsActionChecked::CreateSP(ViewportClientRef, &FContextualAnimViewportClient::IsEntryPosesDrawModeSet, EMultiOptionDrawMode::None));
	}
}

TSharedRef<FEditorViewportClient> SContextualAnimViewport::MakeEditorViewportClient()
{
	ViewportClient = MakeShared<FContextualAnimViewportClient>(PreviewScenePtr.Pin().ToSharedRef(), SharedThis(this), AssetEditorToolkitPtr.Pin().ToSharedRef());
	ViewportClient->ViewportType = LVT_Perspective;
	ViewportClient->bSetListenerPosition = false;
	ViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	ViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	return ViewportClient.ToSharedRef();
}

TSharedPtr<SWidget> SContextualAnimViewport::BuildViewportToolbar()
{
	return SAssignNew(ViewportToolbar, SContextualAnimViewportToolBar, SharedThis(this));
}

TSharedRef<SEditorViewport> SContextualAnimViewport::GetViewportWidget()
{
	return SharedThis(this);
}

TSharedPtr<FExtender> SContextualAnimViewport::GetExtenders() const
{
	TSharedPtr<FExtender> Result(MakeShareable(new FExtender));
	return Result;
}

void SContextualAnimViewport::OnFloatingButtonClicked()
{
}

#undef LOCTEXT_NAMESPACE
