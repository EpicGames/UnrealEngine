// Copyright Epic Games, Inc. All Rights Reserved.

#include "SContextualAnimViewportToolbar.h"
#include "Framework/Commands/UICommandList.h"
#include "SContextualAnimViewport.h"
#include "PreviewProfileController.h"
#include "ContextualAnimAssetEditorCommands.h"

#define LOCTEXT_NAMESPACE "ContextualAnimViewportToolBar"

void SContextualAnimViewportToolBar::Construct(const FArguments& InArgs, TSharedPtr<SContextualAnimViewport> InViewport)
{
	SCommonEditorViewportToolbarBase::Construct(SCommonEditorViewportToolbarBase::FArguments().AddRealtimeButton(false).PreviewProfileController(MakeShared<FPreviewProfileController>()), InViewport);
}

TSharedRef<SWidget> SContextualAnimViewportToolBar::GenerateShowMenu() const
{
	GetInfoProvider().OnFloatingButtonClicked();

	TSharedRef<SEditorViewport> ViewportRef = GetInfoProvider().GetViewportWidget();

	const bool bInShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder ShowMenuBuilder(bInShouldCloseWindowAfterMenuSelection, ViewportRef->GetCommandList());
	{
		ShowMenuBuilder.AddSubMenu(
			LOCTEXT("ShowMenu_IKTargetsDrawSubMenu", "IK Targets"),
			LOCTEXT("ShowMenu_IKTargetsDrawSubMenuToolTip", "IK Targets Drawing Options"),
			FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
				{
					const FContextualAnimAssetEditorCommands& Commands = FContextualAnimAssetEditorCommands::Get();

					SubMenuBuilder.BeginSection("IKTargets", LOCTEXT("ShowMenu_IKTargetsLabel", "IK Targets"));
					{
						SubMenuBuilder.AddMenuEntry(Commands.ShowIKTargetsDrawSelected);
						SubMenuBuilder.AddMenuEntry(Commands.ShowIKTargetsDrawAll);
						SubMenuBuilder.AddMenuEntry(Commands.ShowIKTargetsDrawNone);
					}
					SubMenuBuilder.EndSection();
				})
		);

		ShowMenuBuilder.AddSubMenu(
			LOCTEXT("ShowMenu_SelectionCriteriaDrawSubMenu", "Selection Criteria"),
			LOCTEXT("ShowMenu_SelectionCriteriaDrawSubMenuToolTip", "Selection Criteria Drawing Options"),
			FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
				{
					const FContextualAnimAssetEditorCommands& Commands = FContextualAnimAssetEditorCommands::Get();

					SubMenuBuilder.BeginSection("Selection Criteria", LOCTEXT("ShowMenu_SelectionCriteriaLabel", "Selection Criteria"));
					{
						SubMenuBuilder.AddMenuEntry(Commands.ShowSelectionCriteriaActiveSet);
						SubMenuBuilder.AddMenuEntry(Commands.ShowSelectionCriteriaAllSets);
						SubMenuBuilder.AddMenuEntry(Commands.ShowSelectionCriteriaNone);
					}
					SubMenuBuilder.EndSection();
				})
		);

		ShowMenuBuilder.AddSubMenu(
			LOCTEXT("ShowMenu_EntryPosesDrawSubMenu", "Entry Poses"),
			LOCTEXT("ShowMenu_EntryPosesDrawSubMenuToolTip", "Entry Poses Drawing Options"),
			FNewMenuDelegate::CreateLambda([](FMenuBuilder& SubMenuBuilder)
				{
					const FContextualAnimAssetEditorCommands& Commands = FContextualAnimAssetEditorCommands::Get();

					SubMenuBuilder.BeginSection("Entry Poses", LOCTEXT("ShowMenu_EntryPosesLabel", "Entry Poses"));
					{
						SubMenuBuilder.AddMenuEntry(Commands.ShowEntryPosesActiveSet);
						SubMenuBuilder.AddMenuEntry(Commands.ShowEntryPosesAllSets);
						SubMenuBuilder.AddMenuEntry(Commands.ShowEntryPosesNone);
					}
					SubMenuBuilder.EndSection();
				})
		);
	}

	return ShowMenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
