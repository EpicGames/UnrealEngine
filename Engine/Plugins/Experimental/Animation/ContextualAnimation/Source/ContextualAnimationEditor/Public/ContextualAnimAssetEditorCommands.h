// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Styling/AppStyle.h"

class FContextualAnimAssetEditorCommands : public TCommands<FContextualAnimAssetEditorCommands>
{
public:
	FContextualAnimAssetEditorCommands()
		: TCommands<FContextualAnimAssetEditorCommands>(TEXT("ContextualAnimAssetEditor"), NSLOCTEXT("Contexts", "ContextualAnim", "Contextual Anim"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

public:

	TSharedPtr<FUICommandInfo> ResetPreviewScene;

	TSharedPtr<FUICommandInfo> NewAnimSet;

	TSharedPtr<FUICommandInfo> ShowIKTargetsDrawSelected;
	TSharedPtr<FUICommandInfo> ShowIKTargetsDrawAll;
	TSharedPtr<FUICommandInfo> ShowIKTargetsDrawNone;

	TSharedPtr<FUICommandInfo> ShowSelectionCriteriaActiveSet;
	TSharedPtr<FUICommandInfo> ShowSelectionCriteriaAllSets;
	TSharedPtr<FUICommandInfo> ShowSelectionCriteriaNone;

	TSharedPtr<FUICommandInfo> ShowEntryPosesActiveSet;
	TSharedPtr<FUICommandInfo> ShowEntryPosesAllSets;
	TSharedPtr<FUICommandInfo> ShowEntryPosesNone;

	TSharedPtr<FUICommandInfo> Simulate;
};
