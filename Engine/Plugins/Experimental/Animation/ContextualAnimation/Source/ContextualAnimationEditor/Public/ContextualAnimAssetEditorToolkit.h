// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class FSpawnTabArgs;

class IDetailsView;
class FToolBarBuilder;
class UContextualAnimSceneAsset;
class SContextualAnimViewport;
class SContextualAnimAssetBrowser;
class FContextualAnimPreviewScene;
class FContextualAnimViewModel;
class UContextualAnimPreviewManager;

class FContextualAnimAssetEditorToolkit : public FAssetEditorToolkit
{
public:

	FContextualAnimAssetEditorToolkit();
	virtual ~FContextualAnimAssetEditorToolkit();

	void InitAssetEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, UContextualAnimSceneAsset* SceneAsset);

	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;

	UContextualAnimSceneAsset* GetSceneAsset() const;
	FContextualAnimViewModel* GetViewModel() const { return ViewModel.Get(); }

	void ResetPreviewScene();
	void ToggleSimulateMode();
	bool IsSimulateModeActive() const;
	

private:
	
	bool CanMakeEdits() const;

	TSharedRef<SDockTab> SpawnTab_Viewport(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AssetDetails(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_Timeline(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_AssetBrowser(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnTab_PreviewSettings(const FSpawnTabArgs& Args);

	void BindCommands();
	void ExtendToolbar();
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);

	TSharedRef<SWidget> BuildSectionsMenu();
	TSharedRef<SWidget> BuildAnimSetMenu();

	void ShowNewAnimSetDialog();
	void AddAnimSetToActiveSection();

	TSharedPtr<SContextualAnimViewport> ViewportWidget;

	TSharedPtr<SContextualAnimAssetBrowser> AssetBrowserWidget;

	TSharedPtr<FContextualAnimPreviewScene> PreviewScene;

	TSharedPtr<FContextualAnimViewModel> ViewModel;
};
