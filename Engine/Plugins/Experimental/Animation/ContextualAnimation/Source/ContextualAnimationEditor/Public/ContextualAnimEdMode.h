// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdMode.h"
#include "GenericPlatform/ICursor.h"
#include "HitProxies.h"

class AActor;
class FContextualAnimViewModel;

struct HSelectionCriterionHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	HSelectionCriterionHitProxy(FName InRole, int32 InCriterionIdx, int32 InDataIdx, EHitProxyPriority InPriority = HPP_Wireframe)
		: HHitProxy(InPriority)
	{
		Role = InRole;
		IndexPair.Key = InCriterionIdx;
		IndexPair.Value = InDataIdx;
	}

	virtual EMouseCursor::Type GetMouseCursor() override { return EMouseCursor::Crosshairs; }

	FName Role = NAME_None;
	TPair<int32, int32> IndexPair;
};

class FContextualAnimEdMode : public FEdMode
{
public:

	const static FEditorModeID EdModeId;

	FContextualAnimEdMode();
	virtual ~FContextualAnimEdMode();

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI);
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override;
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData);
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData);

	bool GetHitResultUnderCursor(FHitResult& OutHitResult, FEditorViewportClient* InViewportClient, const FViewportClick& Click) const;

	void DrawIKTargetsForBinding(class FPrimitiveDrawInterface& PDI, const struct FContextualAnimSceneBinding& Binding) const;

private:

	FContextualAnimViewModel* ViewModel = nullptr;
};
