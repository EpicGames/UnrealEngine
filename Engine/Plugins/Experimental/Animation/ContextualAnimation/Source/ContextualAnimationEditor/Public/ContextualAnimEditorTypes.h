// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContextualAnimTypes.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "ContextualAnimEditorTypes.generated.h"

class IPropertyHandle;

class UContextualAnimSceneAsset;

/** Object used to construct the New IK Target Widget */
UCLASS()
class UContextualAnimNewIKTargetParams : public UObject, public IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, Category = "Settings")
	FName SourceRole = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference SourceBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	EContextualAnimIKTargetProvider Provider = EContextualAnimIKTargetProvider::Autogenerated;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (GetOptions = "GetTargetRoleOptions"))
	FName TargetRole = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FBoneReference TargetBone;

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName GoalName = NAME_None;

	UPROPERTY()
	int32 SectionIdx = INDEX_NONE;

	UContextualAnimNewIKTargetParams(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer) {}

	// ~IBoneReferenceSkeletonProvider Interface
	virtual USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle = nullptr) override;

	void Reset(const UContextualAnimSceneAsset& InSceneAsset, const UAnimSequenceBase& InAnimation);

	bool HasValidData() const;

	const UContextualAnimSceneAsset& GetSceneAsset() const;
	int32 GetSectionIdx() const { return SectionIdx; }

	UFUNCTION()
	TArray<FString> GetTargetRoleOptions() const;

private:

	UPROPERTY()
	TWeakObjectPtr<const UContextualAnimSceneAsset> SceneAssetPtr;

	UPROPERTY()
	TArray<FName> CachedRoles;
};

USTRUCT()
struct FContextualAnimNewAnimSetData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "Settings")
	FName RoleName = NAME_None;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TObjectPtr<class UAnimMontage> Animation = nullptr;

	UPROPERTY(EditAnywhere, Category = "Settings")
	TEnumAsByte<enum EMovementMode> MovementMode = EMovementMode::MOVE_Walking;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (EditConditionHides, EditCondition = "MovementMode==EMovementMode::MOVE_Custom"))
	uint8 CustomMovementMode = 0;

	UPROPERTY(EditAnywhere, Category = "Settings")
	bool bOptional = false;

	//@TODO: Refactor this to use FContextualAnimTrack directly with a details customization to hide the properties that are not editable when adding a new set
};

/** Struct used to construct the widget for adding a new set */
USTRUCT()
struct FContextualAnimNewAnimSetParams
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SectionName = NAME_Default;

	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Settings", meta = (TitleProperty = "RoleName"))
	TArray<FContextualAnimNewAnimSetData> Data;

	UPROPERTY(EditAnywhere, Category = "Settings", meta = (ClampMin = "0", UIMin = "0", ClampMax = "1", UIMax = "1"))
	float RandomWeight = 1.f;
};
