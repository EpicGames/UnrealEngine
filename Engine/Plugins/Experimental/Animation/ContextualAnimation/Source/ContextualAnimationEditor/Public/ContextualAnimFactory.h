// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "ContextualAnimFactory.generated.h"

UCLASS()
class UContextualAnimFactory : public UFactory
{
	GENERATED_BODY()

public:
	
	UContextualAnimFactory(const FObjectInitializer& ObjectInitializer);

	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool CanCreateNew() const override;
};
