// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "ContextualAnimMovieSceneNotifySection.generated.h"

struct FAnimNotifyEvent;
class UAnimNotifyState;
class UContextualAnimMovieSceneNotifyTrack;

/** 
 * MovieSceneSection used to represent an AnimNotify in Sequencer Panel 
 * Stores the FGuid of the AnimNotifyEvent this section is presenting 
 */
UCLASS()
class UContextualAnimMovieSceneNotifySection : public UMovieSceneSection
{
	GENERATED_BODY()

public:

	/** 
	 * Initialize this section from the supplied FAnimNotifyEvent. 
	 * Sets the range of the section to match the range of the anim notify and cache the FGuid 
	 */
	void Initialize(const FAnimNotifyEvent& NotifyEvent);

	/** Returns the track owner of this section */
	const FGuid& GetAnimNotifyEventGuid() const { return AnimNotifyEventGuid; }

	/** Returns the track owner of this section */
	UContextualAnimMovieSceneNotifyTrack* GetOwnerTrack() const;

	/** Returns the actual AnimNotifyState this section is representing */
	UAnimNotifyState* GetAnimNotifyState() const;

	/** Returns the actual AnimNotifyEvent this section is representing */
	FAnimNotifyEvent* GetAnimNotifyEvent() const;

private:

	/** Guid of the actual AnimNotifyEvent this section is representing */
	UPROPERTY()
	FGuid AnimNotifyEventGuid;

};
