// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"
#include "UObject/GCObject.h"

struct FBuildEditWidgetParams;

class AActor;
class UAnimMontage;
class USkeleton;
class UContextualAnimSceneAsset;
class UContextualAnimMovieSceneNotifySection;
class UContextualAnimMovieSceneNotifyTrack;
class UContextualAnimMovieSceneSequence;
class UContextualAnimNewIKTargetParams;
class FToolBarBuilder;

/** Handles section drawing and manipulation of a MovieSceneNotifyTrack */
class FContextualAnimMovieSceneNotifyTrackEditor : public FMovieSceneTrackEditor, public FGCObject
{
public:

	FContextualAnimMovieSceneNotifyTrackEditor(TSharedRef<ISequencer> InSequencer);

	// ~FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FContextualAnimMovieSceneNotifyTrackEditor"); }

	// ~FMovieSceneTrackEditor Interface
	virtual void OnInitialize() override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);

	UContextualAnimMovieSceneSequence& GetMovieSceneSequence() const;

private:

	TObjectPtr<UContextualAnimNewIKTargetParams> NewIKTargetParams;

	UContextualAnimMovieSceneNotifySection* CreateNewSection(UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex, UClass* NotifyClass);

	UContextualAnimMovieSceneNotifySection* CreateNewIKSection(UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex, const FName& GoalName);

	void BuildNewIKTargetSubMenu(FMenuBuilder& MenuBuilder, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex);

	void BuildNewIKTargetWidget(FMenuBuilder& MenuBuilder, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex);

	void AddNewNotifyTrack();

	void FillNewNotifyMenu(FMenuBuilder& MenuBuilder, bool bIsAnimNotifyState, UContextualAnimMovieSceneNotifyTrack* Track, int32 RowIndex);

	void CustomizeToolBar(FToolBarBuilder& ToolBarBuilder);
};

// FContextualAnimNotifySection
////////////////////////////////////////////////////////////////////////////////////////////////

/** UI portion of a NotifySection in a NotifyTrack */
class FContextualAnimNotifySection : public FSequencerSection
{
public:
	
	FContextualAnimNotifySection(UMovieSceneSection& InSectionObject)
		: FSequencerSection(InSectionObject)
	{}

	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;
	virtual bool SectionIsResizable() const override;

	UContextualAnimMovieSceneNotifySection* GetNotifySection() const;
	FString GetNotifyName() const;

	static void PaintNotifyName(FSequencerSectionPainter& Painter, int32 LayerId, const FString& InEventString, float PixelPos, bool bIsEventValid);
};
