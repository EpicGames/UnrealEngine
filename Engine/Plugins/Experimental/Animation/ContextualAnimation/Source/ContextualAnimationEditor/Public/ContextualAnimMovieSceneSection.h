// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "ContextualAnimMovieSceneSection.generated.h"

struct FContextualAnimTrack;
class UContextualAnimMovieSceneTrack;

UCLASS()
class UContextualAnimMovieSceneSection : public UMovieSceneSection
{
	GENERATED_BODY()

public:

	void Initialize(int32 InSectionIdx, int32 InAnimSetIdx, int32 InAnimTrackIdx, int32 InSegmentIdx = INDEX_NONE);

	/** Returns the track owner of this section */
	UContextualAnimMovieSceneTrack& GetOwnerTrack() const;

	FContextualAnimTrack& GetAnimTrack() const;

	FORCEINLINE int32 GetSectionIdx() const { return SectionIdx; }
	FORCEINLINE int32 GetAnimSetIdx() const { return AnimSetIdx; }
	FORCEINLINE int32 GetAnimTrackIdx() const { return AnimTrackIdx; }
	FORCEINLINE int32 GetSegmentIdx() const { return SegmentIdx; }

private:

	UPROPERTY()
	int32 SectionIdx = INDEX_NONE;

	UPROPERTY()
	int32 AnimSetIdx = INDEX_NONE;

	UPROPERTY()
	int32 AnimTrackIdx = INDEX_NONE;

	/** Index into the computed timeline segments (INDEX_NONE for single-section mode) */
	UPROPERTY()
	int32 SegmentIdx = INDEX_NONE;
};
