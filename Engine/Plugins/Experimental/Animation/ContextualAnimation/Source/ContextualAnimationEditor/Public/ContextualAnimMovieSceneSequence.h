// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSequence.h"
#include "ContextualAnimMovieSceneSequence.generated.h"

class UMovieScene;
class AActor;
class FContextualAnimViewModel;

UCLASS()
class UContextualAnimMovieSceneSequence : public UMovieSceneSequence
{
	GENERATED_BODY()

public:

	UContextualAnimMovieSceneSequence(const FObjectInitializer& ObjectInitializer);

	void Initialize(const TSharedRef<FContextualAnimViewModel>& ViewModelRef);

	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override;
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override;
	virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	virtual UMovieScene* GetMovieScene() const override;
	virtual UObject* GetParentObject(UObject* Object) const override;
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override;
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override;
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override;
	virtual ETrackSupport IsTrackSupportedImpl(TSubclassOf<class UMovieSceneTrack> InTrackClass) const override;

	FContextualAnimViewModel& GetViewModel() const { return *ViewModelPtr.Pin().Get(); }

private:

	//@TODO: Remove this and use the actors from the PreviewSceneManager
	UPROPERTY()
	TMap<FGuid, TWeakObjectPtr<AActor>> BoundActors;

	/** Pointer to the view model that owns this sequence. */
	TWeakPtr<FContextualAnimViewModel> ViewModelPtr;
};
