// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneNameableTrack.h"
#include "ContextualAnimMovieSceneTrack.generated.h"

enum class EMovieSceneSectionMovedResult;
struct FMovieSceneSectionMovedParams;

class FContextualAnimViewModel;

UCLASS()
class UContextualAnimMovieSceneTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	
	UContextualAnimMovieSceneTrack();

	void Initialize(const FName& InRole);

	FORCEINLINE FName GetRole() const { return Role; }

	FContextualAnimViewModel& GetViewModel() const;

	/** Creates a new section for this track */
	virtual UMovieSceneSection* CreateNewSection() override;
	
	/** Adds a new section to this track */
	virtual void AddSection(UMovieSceneSection& Section) override;
	
	/** Returns whether this track supports the supplied section class */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	
	/** Returns the list of all the sections in this track */
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	
	/** Returns whether this track contains the supplied section */
	virtual bool HasSection(const UMovieSceneSection& Section) const override;

	/** Returns whether this track is empty (doesn't have any section in it) */
	virtual bool IsEmpty() const override;
	
	/** Called when the supplied section is removed from Sequencer. Removes the actual Notify the section is representing */
	virtual void RemoveSection(UMovieSceneSection& Section) override;

	/** Called when a section at the supplied index is removed from Sequencer. Removes the actual Notify the section is representing */
	virtual void RemoveSectionAt(int32 SectionIndex) override;

#if WITH_EDITOR
	/** Called when a section in this track is moved. Updates the actual Notify the section is representing */
	virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params) override;

	virtual FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const override;
	virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const override;

#endif

	virtual bool SupportsMultipleRows() const override { return true; }

private:

	/** List of sections in this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Role this track represents */
	UPROPERTY()
	FName Role = NAME_None;
};
