// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerSection.h"
#include "MovieSceneTrackEditor.h"

struct FBuildEditWidgetParams;

/** Handles section drawing and manipulation of a ContextualAnimMovieSceneTrack */
class FContextualAnimMovieSceneTrackEditor : public FMovieSceneTrackEditor
{
public:

	FContextualAnimMovieSceneTrackEditor(TSharedRef<ISequencer> InSequencer);

	// ~FMovieSceneTrackEditor Interface
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> TrackClass) const override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual void BuildAddTrackMenu(FMenuBuilder& MenuBuilder) override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool GetDefaultExpansionState(UMovieSceneTrack* InTrack) const override { return true; }

	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> InSequencer);
};

// FContextualAnimSection
////////////////////////////////////////////////////////////////////////////////////////////////

/** UI portion of a NotifySection in a NotifyTrack */
class FContextualAnimSection : public FSequencerSection
{
public:
	
	FContextualAnimSection(UMovieSceneSection& InSectionObject)
		: FSequencerSection(InSectionObject)
	{}

	virtual int32 OnPaintSection(FSequencerSectionPainter& InPainter) const override;
	virtual FReply OnSectionDoubleClicked(const FGeometry& SectionGeometry, const FPointerEvent& MouseEvent) override;
	virtual FText GetSectionTitle() const override;
	virtual float GetSectionHeight() const override;
	virtual bool SectionIsResizable() const override { return false; }
};
