// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AdvancedPreviewScene.h"

class FContextualAnimAssetEditorToolkit;

class FContextualAnimPreviewScene : public FAdvancedPreviewScene
{
public:

	FContextualAnimPreviewScene(ConstructionValues CVS, const TSharedRef<FContextualAnimAssetEditorToolkit>& EditorToolkit);
	~FContextualAnimPreviewScene()= default;

	virtual void Tick(float InDeltaTime) override;

	TSharedRef<FContextualAnimAssetEditorToolkit> GetEditorToolkit() const { return EditorToolkitPtr.Pin().ToSharedRef(); }

private:

	/** The asset editor toolkit we are embedded in */
	TWeakPtr<FContextualAnimAssetEditorToolkit> EditorToolkitPtr;
};
