// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene.h"
#include "UObject/GCObject.h"
#include "ContextualAnimTypes.h"
#include "ContextualAnimMovieSceneSequence.h"
#include "EditorUndoClient.h"

class ISequencer;
enum class EMovieSceneDataChangeType;
namespace EMovieScenePlayerStatus { enum Type : int; }

class UWorld;
class FContextualAnimPreviewScene;
class UContextualAnimSceneAsset;
class UContextualAnimManager;
class UContextualAnimMovieSceneSequence;
class UMovieScene;
class UMovieSceneTrack;
class UMovieSceneSection;
class UContextualAnimMovieSceneTrack;
class UContextualAnimMovieSceneNotifyTrack;
class UContextualAnimMovieSceneNotifySection;
class UAnimSequenceBase;
class IDetailsView;
class IStructureDetailsView;
class UContextualAnimNewIKTargetParams;
struct FMovieSceneSectionMovedParams;
struct FContextualAnimNewAnimSetParams;
struct FContextualAnimTrack;

/** Represents a single segment in the flattened multi-section preview timeline */
struct FPreviewTimelineSegment
{
	int32 SectionIdx;           // Index into SceneAsset->Sections
	int32 AnimSetIdx;           // Which anim set to use for this segment
	int32 LoopIteration;        // 0-based loop iteration counter
	float GlobalStartTime;      // Start time in the flattened timeline
	float Duration;             // Max animation length across all tracks in this anim set
};

class FContextualAnimViewModel : public TSharedFromThis<FContextualAnimViewModel>, public FGCObject, public FSelfRegisteringEditorUndoClient
{
public:

	friend class FContextualAnimEdMode;

	enum class ETimelineMode : uint8
	{
		Default,
		Notifies
	};

	enum class ESimulateModeState : uint8
	{
		Inactive,	// Simulate Mode Inactive
		Paused,		// Simulate Mode Active but interaction is not playing
		Playing		// Simulate Mode Active and interaction playing
	};

	FContextualAnimViewModel();
	virtual ~FContextualAnimViewModel();

	// ~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FContextualAnimViewModel"); }

	// ~ FEditorUndoClient
	virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;

	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	void Initialize(UContextualAnimSceneAsset* InSceneAsset, const TSharedRef<FContextualAnimPreviewScene>& InPreviewScene);
	void Shutdown();

	void SetDefaultMode();
	void SetNotifiesMode(const FContextualAnimTrack& AnimTrack);
	ETimelineMode GetTimelineMode() const { return TimelineMode; }
	UAnimSequenceBase* GetEditingAnimation() const { return EditingAnimation.Get(); }
	void RefreshPreviewScene();
	void ResetTimeline();

	TSharedPtr<ISequencer> GetSequencer();
	TSharedPtr<FContextualAnimPreviewScene> GetPreviewScene();
	UMovieScene* GetMovieScene() const { return MovieScene; }
	UContextualAnimMovieSceneSequence* GetMovieSceneSequence() const { return MovieSceneSequence; }
	UContextualAnimSceneAsset* GetSceneAsset() const { return SceneAsset; }
	UContextualAnimMovieSceneTrack* FindTrackByRole(const FName& Role) const;
	TSharedPtr<IDetailsView> GetDetailsView() { return DetailsView; }

	void AddNewAnimSet(const FContextualAnimNewAnimSetParams& Params);
	void RemoveSection(int32 SectionIdx);
	void RemoveAnimSet(int32 SectionIdx, int32 AnimSetIdx);

	void AddNewIKTarget(const UContextualAnimNewIKTargetParams& Params);

	void AnimationModified(UAnimSequenceBase& Animation);

	void SetActiveSection(int32 SectionIdx);
	int32 GetActiveSection() const;
	void SetActiveAnimSetForSection(int32 SectionIdx, int32 AnimSetIdx);
	int32 GetActiveAnimSetForSection(int32 SectionIdx) const;

	/** Returns the section index from the current selection, or the active section if nothing is selected */
	int32 GetDetailsSectionIdx() const;
	/** Returns the anim set index from the current selection, or the active anim set if nothing is selected */
	int32 GetDetailsAnimSetIdx() const;

	void StartSimulateMode();
	void StopSimulateMode();
	void ToggleSimulateMode();
	bool IsSimulateModeInactive()	const { return SimulateModeState == ESimulateModeState::Inactive; }
	bool IsSimulateModePaused()		const { return SimulateModeState == ESimulateModeState::Paused; }
	bool IsSimulateModePlaying()	const { return SimulateModeState == ESimulateModeState::Playing; }
	void StartSimulation();

	/** Called from FContextualAnimEdMode::Tick to poll montage state during multi-section simulate mode */
	void TickSimulateMode(float DeltaTime);

	/** Attempts to build the flattened multi-section preview timeline from the asset's PreviewSequence entries. Returns true if segments were created. */
	bool TryBuildMultiSectionPreviewTimelineData();

	/** Maps a global timeline position to section-local coordinates. Returns false if GlobalTime is out of range. */
	bool ResolveGlobalTime(float GlobalTime, int32& OutSectionIdx, int32& OutAnimSetIdx, float& OutLocalTime, int32& OutSegmentIdx) const;

	/** Returns true if the multi-section preview timeline is active (has segments) */
	bool HasMultiSectionPreviewTimelineData() const { return ComputedTimeline.Num() > 0; }

	/** Transitions preview to a different segment during scrubbing */
	void TransitionPreviewToSegment(int32 SegmentIdx);

	void UpdatePreviewActorTransform(const FContextualAnimSceneBinding& Binding, float Time);

	void UpdateSelection(const AActor* SelectedActor);
	void UpdateSelection(FName Role, int32 CriterionIdx = INDEX_NONE, int32 CriterionDataIdx = INDEX_NONE);
	void UpdateSelection(int32 SectionIdx, int32 AnimSetIdx, int32 AnimTrackIdx);
	void ClearSelection();
	FContextualAnimSceneBinding* GetSelectedBinding() const;
	FContextualAnimTrack* GetSelectedAnimTrack() const;
	AActor* GetSelectedActor() const;
	class UContextualAnimSelectionCriterion* GetSelectedSelectionCriterion() const;
	FText GetSelectionDebugText() const;

	bool ProcessInputDelta(FVector& InDrag, FRotator& InRot, FVector& InScale);
	bool ShouldPreviewSceneDrawWidget() const;
	bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData);
	FVector GetWidgetLocationFromSelection() const;
	bool StartTracking();
	bool EndTracking();

	float GetPlaybackTime() const;
	
	bool CanMakeEdits() const;

	void CacheWarpPoints();
	void UpdateRoles();

private:

	struct FContextualAnimEdSelectionInfo
	{
		/** Selected Role */
		FName Role = NAME_None;

		int32 SectionIdx = INDEX_NONE;

		int32 AnimSetIdx = INDEX_NONE;

		/** Key = Criterion Idx in the AnimTrack, Value = Data idx (e.g vertex idx) */
		TPair<int32, int32> Criterion;

		/** FGuid of the selected AnimNotify */
		FGuid NotifyGuid;

		FContextualAnimEdSelectionInfo()
		{
			Reset();
		}

		void Reset()
		{
			Role = NAME_None;
			SectionIdx = INDEX_NONE;
			AnimSetIdx = INDEX_NONE;
			Criterion.Key = INDEX_NONE;
			Criterion.Value = INDEX_NONE;
			NotifyGuid = FGuid();
		}
	};
	FContextualAnimEdSelectionInfo SelectionInfo;

	/** Scene asset being viewed and edited by this view model. */
	TObjectPtr<UContextualAnimSceneAsset> SceneAsset;

	/** MovieSceneSequence for displaying this scene asset in the sequencer time line. */
	TObjectPtr<UContextualAnimMovieSceneSequence> MovieSceneSequence;

	/** MovieScene for displaying this scene asset in the sequencer time line. */
	TObjectPtr<UMovieScene> MovieScene;

	/** Sequencer instance viewing and editing the scene asset */
	TSharedPtr<ISequencer> Sequencer;

	/** Weak pointer to the PreviewScene */
	TWeakPtr<FContextualAnimPreviewScene> PreviewScenePtr;

	/** Copy of the bindings so we can access them even when simulation mode is not playing (and SceneInstance is not valid) */
	FContextualAnimSceneBindings SceneBindings;

	/** Active section idx */
	int32 ActiveSectionIdx = 0;

	/** Active anim set for each section. Key = SectionIdx, Value = AnimSetIdx  */
	TMap<int32, int32> ActiveAnimSetMap;

	/** The previous play status for sequencer time line. */
	EMovieScenePlayerStatus::Type PreviousSequencerStatus;

	/** The previous time for the sequencer time line. */
	float PreviousSequencerTime = 0.f;

	/** Flattened multi-section preview timeline segments */
	TArray<FPreviewTimelineSegment> ComputedTimeline;

	/** Total duration of the flattened preview sequence */
	float TotalSequenceDuration = 0.f;

	/** Current segment index during scrub/simulate in multi-section mode */
	int32 CurrentSegmentIdx = INDEX_NONE;

	/** Flag for preventing OnAnimNotifyChanged from updating tracks when the change to the animation came from us */
	bool bUpdatingAnimationFromSequencer = false;

	ESimulateModeState SimulateModeState = ESimulateModeState::Inactive;

	ETimelineMode TimelineMode = ETimelineMode::Default;

	TWeakObjectPtr<UAnimSequenceBase> EditingAnimation = nullptr;

	/** Enum to track the current state when modifying the transform of an actor in the scene */
	enum class EModifyActorTransformInSceneState : uint8 {  Inactive, Modifying, WaitingForConfirmation };
	EModifyActorTransformInSceneState ModifyingActorTransformInSceneState = EModifyActorTransformInSceneState::Inactive;

	/** Selected actor when the user starts modifying its transform in the scene */
	TWeakObjectPtr<AActor> ModifyingTransformInSceneCachedActor;

	TSharedPtr<IDetailsView> DetailsView;

	AActor* SpawnPreviewActor(const FContextualAnimTrack& AnimTrack);

	UWorld* GetWorld() const;

	UObject* GetPlaybackContext() const;

	void SequencerTimeChanged();

	void SequencerDataChanged(EMovieSceneDataChangeType DataChangeType);

	void SequencerPlayEvent();

	void SequencerStopEvent();

	void OnSequencerSelectionChangedSections(TArray<UMovieSceneSection*> Sections);

	void OnAnimNotifyChanged(UAnimSequenceBase* Animation);

	void CreateSequencer();

	void CreateDetailsView();

	void RefreshDetailsView();

	bool WantsToModifyMeshToSceneForSelectedActor() const;

private:
	bool bInitialized = false;
};
