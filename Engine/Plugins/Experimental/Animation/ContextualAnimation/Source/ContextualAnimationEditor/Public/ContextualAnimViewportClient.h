// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

class FContextualAnimPreviewScene;
class SContextualAnimViewport;
class FContextualAnimAssetEditorToolkit;

enum class EMultiOptionDrawMode : uint8
{
	None,
	Single,
	All
};

class FContextualAnimViewportClient : public FEditorViewportClient
{
public:

	FContextualAnimViewportClient(const TSharedRef<FContextualAnimPreviewScene>& InPreviewScene, const TSharedRef<SContextualAnimViewport>& InViewport, const TSharedRef<FContextualAnimAssetEditorToolkit>& InAssetEditorToolkit);
	virtual ~FContextualAnimViewportClient()= default;

	// ~FEditorViewportClient interface
	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void TrackingStarted(const struct FInputEventState& InInputState, bool bIsDragging, bool bNudge) override;
	virtual void TrackingStopped() override;
	// ~End of FEditorViewportClient interface

	/** Get the preview scene we are viewing */
	TSharedRef<FContextualAnimPreviewScene> GetPreviewScene() const { return PreviewScenePtr.Pin().ToSharedRef(); }

	TSharedRef<FContextualAnimAssetEditorToolkit> GetAssetEditorToolkit() const { return AssetEditorToolkitPtr.Pin().ToSharedRef(); }

	void SetIKTargetsDrawMode(EMultiOptionDrawMode Mode) { IKTargetsDrawMode = Mode; }
	bool IsIKTargetsDrawModeSet(EMultiOptionDrawMode Mode) const { return IKTargetsDrawMode == Mode; }
	EMultiOptionDrawMode GetIKTargetsDrawMode() const { return IKTargetsDrawMode; }

	void SetSelectionCriteriaDrawMode(EMultiOptionDrawMode Mode) { SelectionCriteriaDrawMode = Mode; }
	bool IsSelectionCriteriaDrawModeSet(EMultiOptionDrawMode Mode) const { return SelectionCriteriaDrawMode == Mode; }
	EMultiOptionDrawMode GetSelectionCriteriaDrawMode() const { return SelectionCriteriaDrawMode; }

	void SetEntryPosesDrawMode(EMultiOptionDrawMode Mode) { EntryPosesDrawMode = Mode; }
	bool IsEntryPosesDrawModeSet(EMultiOptionDrawMode Mode) const { return EntryPosesDrawMode == Mode; }
	EMultiOptionDrawMode GetEntryPosesDrawMode() const { return EntryPosesDrawMode; }

private:

	/** Preview scene we are viewing */
	TWeakPtr<FContextualAnimPreviewScene> PreviewScenePtr;

	/** Asset editor toolkit we are embedded in */
	TWeakPtr<FContextualAnimAssetEditorToolkit> AssetEditorToolkitPtr;

	EMultiOptionDrawMode IKTargetsDrawMode = EMultiOptionDrawMode::None;

	EMultiOptionDrawMode SelectionCriteriaDrawMode = EMultiOptionDrawMode::All;

	EMultiOptionDrawMode EntryPosesDrawMode = EMultiOptionDrawMode::All;
};
