// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class IAssetTypeActions;

class FContextualAnimationEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FContextualAnimationEditorModule& Get();

private:

	FDelegateHandle MovieSceneAnimNotifyTrackEditorHandle;

	TSharedPtr<IAssetTypeActions> ContextualAnimAssetActions;
};
