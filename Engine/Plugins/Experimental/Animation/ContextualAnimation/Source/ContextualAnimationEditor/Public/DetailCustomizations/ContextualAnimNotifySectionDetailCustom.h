// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IDetailsView;
class UContextualAnimMovieSceneNotifySection;

/** Detail Customization for ContextualAnimMovieSceneNotifySection */
class FContextualAnimNotifySectionDetailCustom : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:

	TWeakObjectPtr<UContextualAnimMovieSceneNotifySection> Section;
};
