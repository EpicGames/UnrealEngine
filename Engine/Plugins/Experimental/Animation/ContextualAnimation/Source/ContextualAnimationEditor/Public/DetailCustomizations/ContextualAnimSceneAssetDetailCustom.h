// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class IDetailsView;
class UContextualAnimSceneAsset;
class FContextualAnimViewModel;

class FContextualAnimSceneAssetDetailCustom : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance(TSharedRef<FContextualAnimViewModel> ViewModelRef);

	FContextualAnimSceneAssetDetailCustom(TSharedRef<FContextualAnimViewModel> ViewModelRef);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:

	TWeakPtr<FContextualAnimViewModel> ViewModelPtr;
};
