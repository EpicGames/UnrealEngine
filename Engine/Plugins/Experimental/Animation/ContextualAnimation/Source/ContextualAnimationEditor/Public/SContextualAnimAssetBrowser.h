// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"

struct FAssetData;

class SContextualAnimAssetBrowser : public SBox
{
public:
	
	SLATE_BEGIN_ARGS(SContextualAnimAssetBrowser) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs);
	virtual ~SContextualAnimAssetBrowser()= default;

private:

	TSharedPtr<SBox> AssetBrowserBox;
	void OnAssetDoubleClicked(const FAssetData& AssetData);
	bool OnShouldFilterAsset(const FAssetData& AssetData);
};
