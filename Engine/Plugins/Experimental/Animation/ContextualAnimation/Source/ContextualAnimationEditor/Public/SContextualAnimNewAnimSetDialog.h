// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWindow.h"

class FContextualAnimViewModel;
class FStructOnScope;

/** Dialog for adding a new AnimSet */
class SContextualAnimNewAnimSetDialog : public SWindow
{
public:

	SLATE_BEGIN_ARGS(SContextualAnimNewAnimSetDialog){}
	SLATE_END_ARGS()

	SContextualAnimNewAnimSetDialog() 
		: UserResponse(EAppReturnType::Cancel)
	{
	}

	void Construct(const FArguments& InArgs, const TSharedRef<FContextualAnimViewModel>& InViewModel);

	EAppReturnType::Type Show();

protected:

	TWeakPtr<FContextualAnimViewModel> WeakViewModel;

	EAppReturnType::Type UserResponse;

	TSharedPtr<FStructOnScope> WidgetStruct;

	FReply OnButtonClick(EAppReturnType::Type ButtonID);
};
