// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"
#include "SCommonEditorViewportToolbarBase.h"

class FContextualAnimPreviewScene;
class FContextualAnimViewportClient;
class FContextualAnimAssetEditorToolkit;
class SContextualAnimViewportToolBar;

struct FContextualAnimViewportRequiredArgs
{
	FContextualAnimViewportRequiredArgs(const TSharedRef<FContextualAnimAssetEditorToolkit>& InAssetEditorToolkit, const TSharedRef<FContextualAnimPreviewScene>& InPreviewScene)
		: AssetEditorToolkit(InAssetEditorToolkit)
		, PreviewScene(InPreviewScene)
	{}

	TSharedRef<FContextualAnimAssetEditorToolkit> AssetEditorToolkit;

	TSharedRef<FContextualAnimPreviewScene> PreviewScene;
};

class SContextualAnimViewport : public SEditorViewport, public ICommonEditorViewportToolbarInfoProvider
{
public:
	
	SLATE_BEGIN_ARGS(SContextualAnimViewport) {}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, const FContextualAnimViewportRequiredArgs& InRequiredArgs);
	virtual ~SContextualAnimViewport()= default;

	// ~ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<class SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override;
	// ~End of ICommonEditorViewportToolbarInfoProvider interface

protected:

	// ~SEditorViewport interface
	virtual void BindCommands() override;
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;
	virtual const FSlateBrush* OnGetViewportBorderBrush() const override;
	virtual FSlateColor OnGetViewportBorderColorAndOpacity() const;
	// ~End of SEditorViewport interface

	/** The viewport toolbar */
	TSharedPtr<SContextualAnimViewportToolBar> ViewportToolbar;

	/** Viewport client */
	TSharedPtr<FContextualAnimViewportClient> ViewportClient;

	/** The preview scene that we are viewing */
	TWeakPtr<FContextualAnimPreviewScene> PreviewScenePtr;

	/** Asset editor toolkit we are embedded in */
	TWeakPtr<FContextualAnimAssetEditorToolkit> AssetEditorToolkitPtr;
};
