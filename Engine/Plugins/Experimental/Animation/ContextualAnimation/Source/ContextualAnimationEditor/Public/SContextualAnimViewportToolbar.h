// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SCommonEditorViewportToolbarBase.h"

class SContextualAnimViewport;

class SContextualAnimViewportToolBar : public SCommonEditorViewportToolbarBase
{
public:
	SLATE_BEGIN_ARGS(SContextualAnimViewportToolBar)
	{}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SContextualAnimViewport> InViewport);

	// ~SCommonEditorViewportToolbarBase interface
	virtual TSharedRef<SWidget> GenerateShowMenu() const override;
	// ~End of SCommonEditorViewportToolbarBase interface
};
